{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Data Validation",
	"Description" : "Editor UI and utilities for running data validation",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	
	"Modules" :
	[
		{
			"Name" : "DataValidation",
			"Type": "Editor",
			"LoadingPhase" : "PreDefault"
		}
	]
}