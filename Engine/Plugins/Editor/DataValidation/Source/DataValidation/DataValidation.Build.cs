// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataValidation : ModuleRules
{
	public DataValidation(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"Engine",
				"TargetPlatform",
				"EditorSubsystem",
				"DeveloperSettings"
			}
		);
//		goto through these one by one and remove extra ones
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"AssetRegistry",
				"Json",
				"CollectionManager",
				"ContentBrowser",
				"WorkspaceMenuStructure",
				"AssetTools",
				"PropertyEditor",
				"GraphEditor",
				"LevelEditor",
				"BlueprintGraph",
				"KismetCompiler",
				"SandboxFile",
				"Blutility",
				"ToolMenus",
				"TypedElementRuntime",
				"SourceControl",
				"UncontrolledChangelists",
				"RHI",
				"EditorToolEvents",
				"TelemetryUtils",
			}
		);
	}
}
