// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataValidationChangelist.h"

#include "Algo/AnyOf.h"
#include "Misc/DataValidation.h"
#include "AssetRegistry/AssetData.h"
#include "ISourceControlModule.h"
#include "Misc/PackageName.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "UncontrolledChangelistsModule.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataValidationChangelist)

#define LOCTEXT_NAMESPACE "DataValidationChangelist"

void UDataValidationChangelist::GatherDependencies(const FName& InPackageName, TSet<TPair<FName, FName>>& OutDependencies)
{
	OutDependencies.Add({ InPackageName, InPackageName });

	TArray<FAssetData> Assets;
	TArray<FName> Dependencies;
	USourceControlHelpers::GetAssetDataFromPackage(InPackageName.ToString(), Assets, &Dependencies);

	for (const FName& PackageDependency : Dependencies)
	{
		const FString& PackageDependencyStr = PackageDependency.ToString();

		// Exclude script/memory packages
		if (!FPackageName::IsValidLongPackageName(PackageDependencyStr))
		{
			continue;
		}

		// Exclude Verse dependencies
		if (FPackageName::IsVersePackage(PackageDependencyStr))
		{
			continue;
		}

		// Exclude cooked dependencies
		{
			USourceControlHelpers::GetAssetDataFromPackage(PackageDependencyStr, Assets);
			if (Algo::AnyOf(Assets, [](const FAssetData& Asset) { return Asset.HasAnyPackageFlags(PKG_Cooked); }))
			{
				continue;
			}
		}

		OutDependencies.Add({ InPackageName, PackageDependency });
	}
}

FString UDataValidationChangelist::GetPrettyPackageName(const FName& InPackageName)
{
	TArray<FAssetData> Assets;
	USourceControlHelpers::GetAssetDataFromPackage(InPackageName.ToString(), Assets);

	if (Assets.Num() > 0)
	{
		FString AssetPath = Assets[0].GetObjectPathString();

		int32 LastDot = -1;
		if (AssetPath.FindLastChar('.', LastDot))
		{
			AssetPath.LeftInline(LastDot);
		}

		FString AssetName;

		static FName NAME_ActorLabel(TEXT("ActorLabel"));
		if (Assets[0].FindTag(NAME_ActorLabel))
		{
			Assets[0].GetTagValue(NAME_ActorLabel, AssetName);
		}
		else if (Assets[0].FindTag(FPrimaryAssetId::PrimaryAssetDisplayNameTag))
		{
			Assets[0].GetTagValue(FPrimaryAssetId::PrimaryAssetDisplayNameTag, AssetName);
		}
		else
		{
			AssetName = Assets[0].AssetName.ToString();
		}

		return AssetPath + "." + AssetName;
	}
	else
	{
		return InPackageName.ToString();
	}
}

EDataValidationResult UDataValidationChangelist::IsDataValid(FDataValidationContext& Context) const
{
	// Temporary: do not validate changelists objects on build machines
	// In future we should differentiate between pending and submitted changelists to change the behavior of this function
	if (GIsBuildMachine)
	{
		return EDataValidationResult::Valid;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Gather dependencies of every file in the changelist
	TArray<FName> FilesInChangelist;
	TSet<TPair<FName, FName>> ExternalDependenciesSet;
	
	for (FName PackageName : ModifiedPackageNames)
	{
		FilesInChangelist.Add(PackageName);
		GatherDependencies(PackageName, ExternalDependenciesSet);
	}

	// For every dependency in the external dependencies that is not in the changelist
	// Make sure that the source control state is "not currently modified"
	TArray<TPair<FName, FName>> ExternalDependencies = ExternalDependenciesSet.Array();

	ExternalDependencies.RemoveAll([&FilesInChangelist](TPair<FName, FName>& FilePair) -> bool {
		return Algo::AnyOf(FilesInChangelist, [&FilePair](const auto& FileInChangelist) {
			return FilePair.Value == FileInChangelist;
			});
		});

	bool bHasChangelistErrors = false;

	TArray<FString> ExternalDependenciesFilenames;

	Algo::Transform(ExternalDependencies, ExternalDependenciesFilenames, [](const TPair<FName, FName>& InFilenamePair) -> FString
	{
		return USourceControlHelpers::PackageFilename(InFilenamePair.Value.ToString());
	});

	// Update External dependencies state in case it changed from what is in cache
	SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), ExternalDependenciesFilenames);

	check(ExternalDependenciesFilenames.Num() == ExternalDependencies.Num());

	bool bIgnoreOutOfDateDependencies = false;
	GConfig->GetBool(TEXT("DataValidationChangelistSettings"), TEXT("bIgnoreOutOfDateDependencies"), bIgnoreOutOfDateDependencies, GEditorIni);

	for (int32 i = 0; i < ExternalDependenciesFilenames.Num(); ++i)
	{
		const FString& ExternalPackageFilename = ExternalDependenciesFilenames[i];
		const TPair<FName, FName>& ExternalDependencyPair = ExternalDependencies[i];

		FSourceControlStatePtr ExternalDependencyFileState = SourceControlProvider.GetState(ExternalPackageFilename, EStateCacheUsage::Use);

		// Check if file is in cache; if it's not in the cache, then it's not currently changed.
		if (!ExternalDependencyFileState)
		{
			continue;
		}

		auto GetReferencerText = [&ExternalDependencyPair]() { return FText::FromString(GetPrettyPackageName(ExternalDependencyPair.Key)); };
		auto GetReferenceText = [&ExternalDependencyPair]() { return FText::FromString(GetPrettyPackageName(ExternalDependencyPair.Value)); };
		auto GetExternalPackageFilenameText = [&ExternalPackageFilename]() { return FText::FromString(ExternalPackageFilename); };

		// Dependency is checked out or added but is not in this changelist
		if (ExternalDependencyFileState->IsCheckedOut() || ExternalDependencyFileState->IsAdded())
		{
			bHasChangelistErrors = true;
			FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.Error", "{0} is missing from this changelist (referenced from {1})."), GetReferenceText(), GetReferencerText());
			Context.AddError(CurrentError);
		}
		// Dependency is not at the latest revision
		else if (!ExternalDependencyFileState->IsCurrent())
		{
			if (!bIgnoreOutOfDateDependencies)
			{
				FText CurrentWarning = FText::Format(LOCTEXT("DataValidation.Changelist.NotLatest", "{0} is referenced but is not at the latest revision '{1}' (referenced from {2})"), GetReferenceText(), GetExternalPackageFilenameText(), GetReferencerText());
				Context.AddWarning(CurrentWarning);
			}
		}
		// Dependency is not in source control
		else if (ExternalDependencyFileState->CanAdd())
		{
			if (!FPaths::FileExists(ExternalDependencyFileState->GetFilename()))
			{
				bHasChangelistErrors = true;
				FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.NotInWorkspace", "{0} is referenced and cannot be found in workspace '{1}' (referenced from {2})"), GetReferenceText(), GetExternalPackageFilenameText(), GetReferencerText());
				Context.AddError(CurrentError);
			}
			else
			{
				bHasChangelistErrors = true;
				FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.NotInDepot", "{0} is referenced and must also be added to revision control '{1}' (referenced from {2})"), GetReferenceText(), GetExternalPackageFilenameText(), GetReferencerText());
				Context.AddError(CurrentError);
			}
		}
	}

	if (bHasChangelistErrors)
	{
		FUncontrolledChangelistsModule::Get().OnReconcileAssets();
	}

	return bHasChangelistErrors ? EDataValidationResult::Invalid : EDataValidationResult::Valid;
}

void UDataValidationChangelist::Initialize(FSourceControlChangelistPtr InChangelist)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UDataValidationChangelist::Initialize");
	if (InChangelist.IsValid())
	{
		ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();
		FSourceControlChangelistStatePtr ChangelistStatePtr = Provider.GetState(InChangelist.ToSharedRef(), EStateCacheUsage::Use);
		if (ChangelistStatePtr.IsValid())
		{
			Initialize(ChangelistStatePtr.ToSharedRef());
		}
	}
}

void UDataValidationChangelist::Initialize(FSourceControlChangelistStateRef InChangelist)
{
	Changelist = InChangelist->GetChangelist();	
	Description = InChangelist->GetDescriptionText();
	Initialize(InChangelist->GetFilesStates());
}

void UDataValidationChangelist::Initialize(TConstArrayView<FSourceControlStateRef> FileStates)
{
	for (const FSourceControlStateRef& FileState : FileStates)
	{
		FString Filename = FileState->GetFilename();	
		if (FPackageName::IsPackageFilename(Filename))
		{
			FString PackageName;
			FString FailureReason;
			if (FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName, &FailureReason))
			{
				if (FileState->IsDeleted())
				{
					DeletedPackageNames.Add(FName(PackageName));
				}
				else
				{
					ModifiedPackageNames.Add(FName(PackageName));
				}
			}
			continue;
		}
		
		// If it wasn't an asset file or we failed to find the relevant package name 
		if (FileState->IsDeleted())
		{
			DeletedFiles.Add(MoveTemp(Filename));
		}
		else 
		{
			ModifiedFiles.Add(MoveTemp(Filename));
		}
	}
}

#undef LOCTEXT_NAMESPACE
