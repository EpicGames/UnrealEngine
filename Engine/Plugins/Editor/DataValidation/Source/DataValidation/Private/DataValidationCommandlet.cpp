// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataValidationCommandlet.h"

#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprint/BlueprintSupport.h"
#include "DataValidationModule.h"
#include "Editor.h"
#include "EditorUtilityBlueprint.h"
#include "EditorValidatorBase.h"
#include "EditorValidatorSubsystem.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataValidationCommandlet)


DEFINE_LOG_CATEGORY_STATIC(LogDataValidation, Warning, All);

// Commandlet for validating data
int32 UDataValidationCommandlet::Main(const FString& FullCommandLine)
{
	// This commandlet won't work properly when use outside of an editor executable
	check(GEditor);

	UE_LOGF(LogDataValidation, Log, "--------------------------------------------------------------------------------------------");
	UE_LOGF(LogDataValidation, Log, "Running %ls Commandlet", *GetClass()->GetName());

	// validate data
	if (!ValidateDataImpl(FullCommandLine))
	{
		UE_LOGF(LogDataValidation, Warning, "Errors occurred while validating data");
		return 2; // return something other than 1 for error since the engine will return 1 if any other system (possibly unrelated) logged errors during execution.
	}

	UE_LOGF(LogDataValidation, Log, "Successfully finished running %ls Commandlet", *GetClass()->GetName());
	UE_LOGF(LogDataValidation, Log, "--------------------------------------------------------------------------------------------");
	return 0;
}

//static
bool UDataValidationCommandlet::ValidateData(const FString& FullCommandLine)
{
	const UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>();
	check(EditorValidationSubsystem);

	// Attempt to load the data validation commandlet class from the EditorValidationSubsystem's configured
	// GetDataValidationCommandletClassName property, but if it is not specified then fall back on the UDataValidationCommandlet.
	UClass* CommandletClass = UDataValidationCommandlet::StaticClass();
	if (!EditorValidationSubsystem->GetDataValidationCommandletClassName().IsEmpty())
	{
		UClass* ConfiguredClass = FindObject<UClass>(nullptr, *EditorValidationSubsystem->GetDataValidationCommandletClassName());
		if (ConfiguredClass == nullptr)
		{
			UE_LOGF(LogDataValidation, Error, "Unable to load Data Validation Commandlet Class: %ls", *EditorValidationSubsystem->GetDataValidationCommandletClassName());
			return false;
		}
		else if (!ConfiguredClass->IsChildOf(UDataValidationCommandlet::StaticClass()))
		{
			UE_LOGF(LogDataValidation, Error, "Data Validation Commandlet Class Name is not a UDataValidationCommandlet class: %ls", *EditorValidationSubsystem->GetDataValidationCommandletClassName());
			return false;
		}
		else
		{
			CommandletClass = ConfiguredClass;
		}
	}

	TStrongObjectPtr<UDataValidationCommandlet> Commandlet{ NewObject<UDataValidationCommandlet>(GetTransientPackage(), CommandletClass) };
	if (Commandlet)
	{
		return Commandlet->ValidateDataImpl(FullCommandLine);
	}
	else
	{
		UE_LOGF(LogDataValidation, Error, "Data Validation Commandlet can be instanciated. The UClass might be abstract. Commandlet Class Name: %ls", *EditorValidationSubsystem->GetDataValidationCommandletClassName());
		return false;
	}
}

bool UDataValidationCommandlet::ValidateDataImpl(const FString& FullCommandLine)
{
	ProcessCommandLine(FullCommandLine);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName);
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	WaitForAssetRegistry();

	TArray<FAssetData> AssetDataList;
	if (!GetAssetsToValidate(AssetRegistry, AssetDataList))
	{
		return false;
	}

	FilterAssetsToValidate(AssetDataList);

	if (!GEditor->IsInitialized() &&
		ShouldLoadDefaultEditorModules(AssetDataList))
	{
		GEditor->LoadDefaultEditorModules();
	}

	FValidateAssetsSettings Settings;
	SetupValidationSettings(Settings);

	FValidateAssetsResults Results;
	ValidateAssets(AssetDataList, Settings, Results);

	return ProcessValidationResults(Results);
}

bool UDataValidationCommandlet::ShouldLoadDefaultEditorModules(const TArray<FAssetData>& AssetDataList)
{
	// Check if we have some BP validator that were created using an editor utility.
	// Those Editor Utilities Validator might have an dependency to an editor module that is loaded during the editor initialization.
	const FTopLevelAssetPath EditorUtilityClassPath = UEditorUtilityBlueprint::StaticClass()->GetClassPathName();
	const FString EditorValidatorBaseClassExportPath = FObjectPropertyBase::GetExportPath(UEditorValidatorBase::StaticClass());
	return AssetDataList.ContainsByPredicate([EditorUtilityClassPath, &EditorValidatorBaseClassExportPath](const FAssetData& AssetData)
		{
			if (AssetData.AssetClassPath == EditorUtilityClassPath)
			{
				if (AssetData.TagsAndValues.ContainsKeyValue(FBlueprintTags::NativeParentClassPath, EditorValidatorBaseClassExportPath))
				{
					return true;
				}
			}

			return false;
		});
}

void UDataValidationCommandlet::ProcessCommandLine(const FString& FullCommandLine)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Params;
	ParseCommandLine(*FullCommandLine, Tokens, Switches, Params);

	ProcessCommandLine(Tokens, Switches, Params);
}

void UDataValidationCommandlet::ProcessCommandLine(const TArray<FString>& Tokens, const TArray<FString>& Switches, const TMap<FString, FString>& Params)
{
	AssetTypeFilterString = Params.FindRef(TEXT("AssetType"));
	bIncludeOnlyOnDiskAssets = Switches.Contains(TEXT("IncludeOnlyOnDiskAssets"));
	bWithoutEngine = !Switches.Contains(TEXT("includeengine"));
}

bool UDataValidationCommandlet::GetAssetsToValidate(IAssetRegistry& AssetRegistry, TArray<FAssetData>& OutAssetDataList)
{
	if (!AssetTypeFilterString.IsEmpty())
	{
		if (FPackageName::IsShortPackageName(AssetTypeFilterString))
		{
			UClass* Class = FindFirstObject<UClass>(*AssetTypeFilterString, EFindFirstObjectOptions::EnsureIfAmbiguous);
			if (Class)
			{
				AssetTypeFilterString = Class->GetPathName();
			}
			else
			{
				UE_LOGF(LogDataValidation, Error, "Unable to resolve class path name given short name: \"%ls\"", *AssetTypeFilterString);
				return false;
			}
		}
	
		FARFilter Filter;
		Filter.ClassPaths.Add(FTopLevelAssetPath(AssetTypeFilterString));
		Filter.bRecursiveClasses = true;
		Filter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;
		AssetRegistry.GetAssets(Filter, OutAssetDataList);
	}
	else
	{
		AssetRegistry.GetAllAssets(OutAssetDataList, bIncludeOnlyOnDiskAssets);
	}

	return true;
}

void UDataValidationCommandlet::FilterAssetsToValidate(TArray<FAssetData>& AssetDataList)
{
	if (bWithoutEngine)
	{
		FString EngineDir = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());
		AssetDataList.RemoveAll([&EngineDir](const FAssetData& AssetData)
			{
				// Remove /Engine and any plugins from /Engine, but keep /Game and any plugins under /Game.
				FString FileName;
				FString PackageName;
				AssetData.PackageName.ToString(PackageName);
				if (!FPackageName::TryConvertLongPackageNameToFilename(PackageName, FileName))
				{
					// We don't recognize this packagepath, so keep it
					return false;
				}
				// ConvertLongPackageNameToFilename can return ../../Plugins for some plugins instead of
				// ../../../Engine/Plugins. We should fix that in FPackageName to always return the normalized
				// filename. For now, workaround it by converting to absolute paths.
				FileName = FPaths::ConvertRelativePathToFull(MoveTemp(FileName));
				return FPathViews::IsParentPathOf(EngineDir, FileName);
			});
	}
}

void UDataValidationCommandlet::SetupValidationSettings(FValidateAssetsSettings& Settings)
{
	Settings.bSkipExcludedDirectories = true;
	Settings.bShowIfNoFailures = true;
	Settings.ValidationUsecase = EDataValidationUsecase::Commandlet;
}

void UDataValidationCommandlet::ValidateAssets(
	const TArray<FAssetData>& AssetDataList,
	const FValidateAssetsSettings& InSettings,
	FValidateAssetsResults& OutResults)
{
	UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>();
	check(EditorValidationSubsystem);
	EditorValidationSubsystem->ValidateAssetsWithSettings(AssetDataList, InSettings, OutResults);
}

bool UDataValidationCommandlet::ProcessValidationResults(FValidateAssetsResults& Results)
{
	// Only return false if the commandlet couldn't run the validation and not if there was error in found by the validation
	return true;
}

void UDataValidationCommandlet::WaitForAssetRegistry()
{
	constexpr bool bSynchronousSearch = true;
	IAssetRegistry::GetChecked().SearchAllAssets(bSynchronousSearch);
}
