// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataValidationFixers.h"

#include "Editor.h"
#include "EditorValidatorSubsystem.h"
#include "FileHelpers.h"
#include "Algo/AnyOf.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "DataValidationFixers"

namespace UE::DataValidation
{

EFixApplicability FSingleUseFixer::GetApplicability(int32 FixIndex) const
{
	if (UsedFixes.Contains(FixIndex))
	{
		return EFixApplicability::Applied;
	}
	else
	{
		return Inner->GetApplicability(FixIndex);
	}
}

FFixResult FSingleUseFixer::ApplyFix(int32 FixIndex)
{
	FFixResult Result = Inner->ApplyFix(FixIndex);
	UsedFixes.Add(FixIndex);
	return Result;
}

TSharedRef<FSingleUseFixer> FSingleUseFixer::Create(TSharedRef<IFixer> Inner)
{
	TSharedRef<FSingleUseFixer> Fixer = MakeShared<FSingleUseFixer>();
	Fixer->Inner = MoveTemp(Inner);
	return Fixer;
}

EFixApplicability FObjectSetDependentFixer::GetApplicability(int32 FixIndex) const
{
	if (Algo::AnyOf(Dependencies, [](const TWeakObjectPtr<>& Ptr) { return Ptr.IsStale(); }))
	{
		return EFixApplicability::DidNotApply;
	}
	else
	{
		return Inner->GetApplicability(FixIndex);
	}
}

FFixResult FObjectSetDependentFixer::ApplyFix(int32 FixIndex)
{
	return Inner->ApplyFix(FixIndex);
}

TSharedRef<FObjectSetDependentFixer> FObjectSetDependentFixer::Create(TSharedRef<IFixer> Inner, TArray<TWeakObjectPtr<>> Dependencies)
{
	TSharedRef<FObjectSetDependentFixer> Fixer = MakeShared<FObjectSetDependentFixer>();
	Fixer->Inner = MoveTemp(Inner);
	Fixer->Dependencies = MoveTemp(Dependencies);
	return Fixer;
}

EFixApplicability FAutoSavingFixer::GetApplicability(int32 FixIndex) const
{
	return Inner->GetApplicability(FixIndex);
}

template <typename F>
auto CollectDirtyPackagesDuring(TSet<UPackage*>& OutPackagesMarkedDirty, F&& Lambda) -> decltype(Lambda())
{
	FDelegateHandle PackageModificationListenerHandle = UPackage::PackageMarkedDirtyEvent.AddLambda(
		// Lambda capture safety: This lambda is removed from the delegate before PackagesMarkedDirty goes out of scope.
		[&OutPackagesMarkedDirty](UPackage* Package, bool bWasDirty)
		{
			if (Package)
			{
				OutPackagesMarkedDirty.Add(Package);
			}
		}
	);

	auto Result = Lambda();

	UPackage::PackageMarkedDirtyEvent.Remove(PackageModificationListenerHandle);

	return Result;
}

FFixResult FAutoSavingFixer::ApplyFix(int32 FixIndex)
{
	TSet<UPackage*> PackagesMarkedDirty;
	FFixResult FixResult =
		CollectDirtyPackagesDuring(PackagesMarkedDirty, [this, FixIndex] { return Inner->ApplyFix(FixIndex); });

	if (FixResult.bIsSuccess && !PackagesMarkedDirty.IsEmpty())
	{
		TArray<UPackage*> PackagesToSave = PackagesMarkedDirty.Array();
		FEditorFileUtils::FPromptForCheckoutAndSaveParams Params;
		Params.Title = LOCTEXT("SaveFixedAssets", "Save assets after applying fix");
		Params.Message = LOCTEXT(
			"SaveFixedAssetsDescription", "Applying the fix modified the following assets, which now need to be saved."
		);
		// NOTE: Return code ignored because ultimately the automatic save is just a nice thing we do for the user.
		// It's not mandatory for the assets to be saved.
		[[maybe_unused]] FEditorFileUtils::EPromptReturnCode PromptReturnCode =
			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, Params);
	}

	return FixResult;
}

TSharedRef<FAutoSavingFixer> FAutoSavingFixer::Create(TSharedRef<IFixer> Inner)
{
	TSharedRef<FAutoSavingFixer> Fixer = MakeShared<FAutoSavingFixer>();
	Fixer->Inner = MoveTemp(Inner);
	return Fixer;
}

EFixApplicability FValidatingFixer::GetApplicability(int32 FixIndex) const
{
	return Inner->GetApplicability(FixIndex);
}

FFixResult FValidatingFixer::ApplyFix(int32 FixIndex)
{
	TSet<UPackage*> PackagesMarkedDirty;
	FFixResult FixResult =
		CollectDirtyPackagesDuring(PackagesMarkedDirty, [this, FixIndex] { return Inner->ApplyFix(FixIndex); });

	if (FixResult.bIsSuccess && !PackagesMarkedDirty.IsEmpty())
	{
		if (GEditor != nullptr)
		{
			UEditorValidatorSubsystem* ValidatorSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>();
			TArray<FAssetData> AssetsToValidate;
			AssetsToValidate.Reserve(PackagesMarkedDirty.Num());
			for (const UPackage* Package : PackagesMarkedDirty)
			{
				AssetsToValidate.Add(FAssetData(Package));
			}
			FValidateAssetsSettings ValidationSettings;
			ValidationSettings.ValidationUsecase = EDataValidationUsecase::Save;
			FValidateAssetsResults ValidationResults;
			ValidatorSubsystem->ValidateAssetsWithSettings(AssetsToValidate, ValidationSettings, ValidationResults);
		}
	}

	return FixResult;
}

TSharedRef<FValidatingFixer> FValidatingFixer::Create(TSharedRef<IFixer> Inner)
{
	TSharedRef<FValidatingFixer> Fixer = MakeShared<FValidatingFixer>();
	Fixer->Inner = MoveTemp(Inner);
	return Fixer;
}

FMutuallyExclusiveFixSet::FMutuallyExclusiveFixSet()
{
	SharedData = MakeShared<FSharedData>();
}

void FMutuallyExclusiveFixSet::Add(const FText& Label, TSharedRef<IFixer> Inner)
{
	TSharedRef<FFixer> Fixer = MakeShared<FFixer>();
	Fixer->Inner = MoveTemp(Inner);
	Fixer->SharedData = SharedData;
	QueuedTokens.Add(FQueuedToken{.Label = Label, .Fixer = MoveTemp(Fixer)});
}

void FMutuallyExclusiveFixSet::Transform(const TFunctionRef<TSharedRef<IFixer>(TSharedRef<IFixer>)>& Callback)
{
	for (FQueuedToken& QueuedToken : QueuedTokens)
	{
		QueuedToken.Fixer->Inner = Callback(QueuedToken.Fixer->Inner.ToSharedRef());
	}
}

void FMutuallyExclusiveFixSet::CreateTokens(const TFunctionRef<void(TSharedRef<FFixToken>)>& Callback) const
{
	for (int32 i = 0; i < QueuedTokens.Num(); ++i)
	{
		const FQueuedToken& QueuedToken = QueuedTokens[i];
		FText Label = i == 0 ? LOCTEXT("FirstMutuallyExclusiveFix", "Fix: {0}") : LOCTEXT("NextMutuallyExclusiveFix", " or: {0}");
		Callback(FFixToken::Create(FText::Format(Label, QueuedToken.Label), QueuedToken.Fixer, i));
	}
}

EFixApplicability FMutuallyExclusiveFixSet::FFixer::GetApplicability(int32 FixIndex) const
{
	if (SharedData->AppliedFix == INDEX_NONE)
	{
		return Inner->GetApplicability(0);
	}

	if (SharedData->AppliedFix == FixIndex)
	{
		return EFixApplicability::Applied;
	}
	else
	{
		return EFixApplicability::DidNotApply;
	}
}

FFixResult FMutuallyExclusiveFixSet::FFixer::ApplyFix(int32 FixIndex)
{
	FFixResult Result = Inner->ApplyFix(0);
	SharedData->AppliedFix = FixIndex;
	return Result;
}

}

#undef LOCTEXT_NAMESPACE
