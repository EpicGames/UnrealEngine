// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataValidationModule.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetDataToken.h"
#include "UObject/ObjectSaveContext.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "ToolMenus.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/MessageDialog.h"
#include "DataValidationCommandlet.h"
#include "EditorValidatorBase.h"
#include "EditorValidator_Material.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "HAL/IConsoleManager.h"
#include "LoadExternalObjectsForValidation.h"
#include "Logging/MessageLog.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "UObject/ICookInfo.h"

#include "ContentBrowserMenuContexts.h"
#include "LevelEditorMenuContext.h"

#include "EditorValidatorHelpers.h"
#include "EditorValidatorSubsystem.h"
#include "ISettingsModule.h"
#include "Algo/RemoveIf.h"
#include "Misc/PackageName.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "DataValidationModule"

namespace DataValidationOptions
{
	static bool bReportCookValidationStats = true;
	static FAutoConsoleVariableRef CVarReportCookValidationStats(
		TEXT("DataValidation.ReportCookValidationStats"),
		bReportCookValidationStats,
		TEXT("True to report post-cook statistics for each validator"));
}

class FDataValidationModule : public IDataValidationModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface

	/** Validates selected assets and opens a window to report the results. If bValidateDependencies is true it will also validate any assets that the selected assets depend on. */
	virtual void ValidateAssets(const TArray<FAssetData>& SelectedAssets, bool bValidateDependencies, const EDataValidationUsecase InValidationUsecase) override;

	void ValidateFolders(const TArray<FString>& SelectedFolders, const EDataValidationUsecase InValidationUsecase);

private:
	void OnPackageSaved(const FString& PackageFileName, UPackage* Package, FObjectPostSaveContext ObjectSaveContext);

	void OnCookStarted(UE::Cook::ICookInfo& CookInfo);
	void OnCookFinished(UE::Cook::ICookInfo& CookInfo);
#if ENABLE_COOK_STATS
	void LogCookStats(FCookStatsManager::AddStatFuncRef AddStat);
#endif
	EDataValidationResult OnValidateSourcePackageDuringCook(UPackage* Package, FDataValidationContext& ValidationContext);

	// Adds Asset and any assets it depends on to the set DependentAssets
	void FindAssetDependencies(const FAssetRegistryModule& AssetRegistryModule, const FAssetData& Asset, TSet<FAssetData>& DependentAssets);

	void RegisterMenus();
	static FText Menu_ValidateDataGetTitle();
	static void Menu_ValidateData();

	struct FCookValidationStats
	{
		TMap<FTopLevelAssetPath, FValidatorStatistics> ValidatorStats;
		double ExternalObjectLoadTotalDurationSeconds = 0.0;
		double ExternalObjectUnloadTotalDurationSeconds = 0.0;
		int32 ExternalObjectTotalCount = 0;
	};
	TOptional<FCookValidationStats> CookValidationStats;
};

IMPLEMENT_MODULE(FDataValidationModule, DataValidation)

void FDataValidationModule::StartupModule()
{	
	UE::Cook::FDelegates::CookStarted.AddRaw(this, &FDataValidationModule::OnCookStarted);
	UE::Cook::FDelegates::CookFinished.AddRaw(this, &FDataValidationModule::OnCookFinished);
#if ENABLE_COOK_STATS
	FCookStatsManager::CookStatsCallbacks.AddRaw(this, &FDataValidationModule::LogCookStats);
#endif
	UE::Cook::FDelegates::ValidateSourcePackage.BindRaw(this, &FDataValidationModule::OnValidateSourcePackageDuringCook);

	if (!IsRunningCommandlet() && !IsRunningGame() && FSlateApplication::IsInitialized())
	{
		// add the File->DataValidation menu subsection
		UToolMenus::Get()->RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FDataValidationModule::RegisterMenus));

		// Add save callback
		UPackage::PackageSavedWithContextEvent.AddRaw(this, &FDataValidationModule::OnPackageSaved);

		ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
		SettingsModule.RegisterSettings("Editor", "Advanced", "DataValidation",
			LOCTEXT("DataValidationName", "Data Validation"),
			LOCTEXT("DataValidationDescription", "Settings related to validating assets in the editor."),
			GetMutableDefault<UDataValidationSettings>()
		);

		FMaterialEditorValidationShaderPlatform::RegisterCustomPropertyTypeLayout();
	}
}

void FDataValidationModule::ShutdownModule()
{
	UE::Cook::FDelegates::CookStarted.RemoveAll(this);
	UE::Cook::FDelegates::CookFinished.RemoveAll(this);
#if ENABLE_COOK_STATS
	FCookStatsManager::CookStatsCallbacks.RemoveAll(this);
#endif
	UE::Cook::FDelegates::ValidateSourcePackage.Unbind();

	if (!IsRunningCommandlet() && !IsRunningGame() && !IsRunningDedicatedServer())
	{
		// remove menu extension
		UToolMenus::UnregisterOwner(this);

		UPackage::PackageSavedWithContextEvent.RemoveAll(this);

		FMaterialEditorValidationShaderPlatform::UnregisterCustomPropertyTypeLayout();
	}
}

void FDataValidationModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools");
		FToolMenuSection& Section = Menu->AddSection("DataValidation", LOCTEXT("DataValidation", "Data Validation"));
		Section.AddEntry(FToolMenuEntry::InitMenuEntry(
			"ValidateData",
			TAttribute<FText>::Create(&Menu_ValidateDataGetTitle),
			LOCTEXT("ValidateDataTooltip", "Validates all user data in content directory."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DeveloperTools.MenuIcon"),
			FUIAction(FExecuteAction::CreateStatic(&FDataValidationModule::Menu_ValidateData))
		));
	}

	{
		FToolUIAction Action;
		Action.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
		{
			if (ULevelEditorContextMenuContext* Context = InContext.FindContext<ULevelEditorContextMenuContext>();
				Context && Context->CurrentSelection)
			{
				TArray<FAssetData> SelectedActorAssets;
				Context->CurrentSelection->ForEachSelectedObject<AActor>([&SelectedActorAssets](AActor* SelectedActor)
				{
					if (SelectedActor->GetExternalPackage())
					{
						SelectedActorAssets.Add(FAssetData(SelectedActor));
					}
					return true;
				});
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
				IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
				for (FAssetData& AssetData : SelectedActorAssets)
				{
					FAssetData DiskAssetData = AssetRegistry.GetAssetByObjectPath(AssetData.GetSoftObjectPath(), true);
					if (DiskAssetData.IsValid())
					{
						AssetData = MoveTemp(DiskAssetData);
					}
				}
				ValidateAssets(SelectedActorAssets, false, EDataValidationUsecase::Manual);
			}
		});
		Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
		{
			bool bCanValidateActor = false;
			if (ULevelEditorContextMenuContext* Context = InContext.FindContext<ULevelEditorContextMenuContext>();
				Context && Context->CurrentSelection)
			{
				Context->CurrentSelection->ForEachSelectedObject<AActor>([&bCanValidateActor](AActor* SelectedActor)
				{
					bCanValidateActor |= SelectedActor->GetExternalPackage() != nullptr; // Only allow validation of OFPA packages
					return !bCanValidateActor;
				});
			}
			return bCanValidateActor;
		});

		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu");
		FToolMenuSection& Section = Menu->FindOrAddSection("ActorOptions");
		Section.AddMenuEntry(
			"ValidateActors",
			LOCTEXT("ValidateActorsTabTitle", "Validate"),
			LOCTEXT("ValidateActorsTooltipText", "Runs data validation on these actors."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "DeveloperTools.MenuIcon"),
			Action
		);
	}

	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.AssetActionsSubMenu");
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
		{
			const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
			if (Context && Context->bCanBeModified && Context->SelectedAssets.Num() > 0)
			{
				FToolMenuSection& Section = InMenu->FindOrAddSection("AssetContextAdvancedActions");
				Section.AddMenuEntry(
					"ValidateAssets",
					LOCTEXT("ValidateAssetsTabTitle", "Validate Assets"),
					LOCTEXT("ValidateAssetsTooltipText", "Runs data validation on these assets."),
					FSlateIcon(),
					FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = InContext.FindContext<UContentBrowserAssetContextMenuContext>())
						{
							ValidateAssets(Context->SelectedAssets, false, EDataValidationUsecase::Manual);
						}
					})
				);

				Section.AddMenuEntry(
					"ValidateAssetsAndDependencies",
					LOCTEXT("ValidateAssetsAndDependenciesTabTitle", "Validate Assets and Dependencies"),
					LOCTEXT("ValidateAssetsAndDependenciesTooltipText", "Runs data validation on these assets and all assets they depend on."),
					FSlateIcon(),
					FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
					{
						if (const UContentBrowserAssetContextMenuContext* Context = InContext.FindContext<UContentBrowserAssetContextMenuContext>())
						{
							ValidateAssets(Context->SelectedAssets, true, EDataValidationUsecase::Manual);
						}
					})
				);
			}
		}));
	}

	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu");
		FToolMenuSection& Section = Menu->FindOrAddSection("PathContextBulkOperations");
		Section.AddMenuEntry(
			"ValidateAssetsPath",
			LOCTEXT("ValidateAssetsPathTabTitle", "Validate Assets in Folder"),
			LOCTEXT("ValidateAssetsPathTooltipText", "Runs data validation on the assets in the selected folder."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Validate"),
			FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
			{
				if (const UContentBrowserFolderContext* Context = InContext.FindContext<UContentBrowserFolderContext>())
				{
					const TArray<FString>& SelectedPaths = Context->GetSelectedPackagePaths();

					FString FormattedSelectedPaths;
					for (int32 i = 0; i < SelectedPaths.Num(); ++i)
					{
						FormattedSelectedPaths.Append(SelectedPaths[i]);
						if (i < SelectedPaths.Num() - 1)
						{
							FormattedSelectedPaths.Append(LINE_TERMINATOR);
						}
					}
					FFormatNamedArguments Args;
					Args.Add(TEXT("Paths"), FText::FromString(FormattedSelectedPaths));
					const EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(LOCTEXT("DataValidationConfirmation", "Are you sure you want to proceed with validating the following folders?\n\n{Paths}"), Args));
					if (Result == EAppReturnType::Yes)
					{
						ValidateFolders(SelectedPaths, EDataValidationUsecase::Manual);
					}
				}
			})
		);
	}
}

FText FDataValidationModule::Menu_ValidateDataGetTitle()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		return LOCTEXT("ValidateDataTitleDA", "Validate Data [Discovering Assets]");
	}
	return LOCTEXT("ValidateDataTitle", "Validate Data...");
}

void FDataValidationModule::Menu_ValidateData()
{
	// make sure the asset registry is finished building
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("AssetsStillScanningError", "Cannot run data validation while still discovering assets."));
		return;
	}

	// validate the data
	bool bSuccess = UDataValidationCommandlet::ValidateData(FString());

	// display an error if the task failed
	if (!bSuccess)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DataValidationError", "An error was encountered during data validation. See the log for details."));
		return;
	}
}

void FDataValidationModule::FindAssetDependencies(const FAssetRegistryModule& AssetRegistryModule, const FAssetData& Asset, TSet<FAssetData>& DependentAssets)
{
	if (Asset.IsValid())
	{
		UObject* Obj = Asset.GetAsset();
		if (Obj)
		{
			const FName SelectedPackageName = Obj->GetOutermost()->GetFName();
			FString PackageString = SelectedPackageName.ToString();
			FString ObjectString = FString::Printf(TEXT("%s.%s"), *PackageString, *FPackageName::GetLongPackageAssetName(PackageString));

			if (!DependentAssets.Contains(Asset))
			{
				DependentAssets.Add(Asset);

				TArray<FName> Dependencies;
				AssetRegistryModule.Get().GetDependencies(SelectedPackageName, Dependencies, UE::AssetRegistry::EDependencyCategory::Package);

				for (const FName& Dependency : Dependencies)
				{
					const FString DependencyPackageString = Dependency.ToString();
					FString DependencyObjectString = FString::Printf(TEXT("%s.%s"), *DependencyPackageString, *FPackageName::GetLongPackageAssetName(DependencyPackageString));

					// recurse on each dependency
					FAssetData DependentAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(DependencyObjectString));

					FindAssetDependencies(AssetRegistryModule, DependentAsset, DependentAssets);
				}
			}
		}
	}
}

void FDataValidationModule::ValidateAssets(const TArray<FAssetData>& SelectedAssets, bool bValidateDependencies, const EDataValidationUsecase InValidationUsecase)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TSet<FAssetData> DependentAssets;

	if (bValidateDependencies)
	{
		for (const FAssetData& Asset : SelectedAssets)
		{
			FindAssetDependencies(AssetRegistryModule, Asset, DependentAssets);
		}
	}

	UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>();
	if (EditorValidationSubsystem)
	{
		FValidateAssetsSettings Settings;
		Settings.bSkipExcludedDirectories = false;
		Settings.bShowIfNoFailures = true;
		Settings.ValidationUsecase = InValidationUsecase;
		Settings.MessageLogPageTitle = UE::DataValidation::MakeTimestampedMessageLogPageTitle(FText::Format(LOCTEXT("MessageLogPageTitle.ValidateSelectedAssets", "Asset Validation: {0}"), SelectedAssets.Num() == 1 ? FText::FromName(SelectedAssets[0].AssetName) : LOCTEXT("MultipleAssets", "multiple assets")));

		if (InValidationUsecase == EDataValidationUsecase::Manual)
		{
			Settings.bUnloadAssetsLoadedForValidation = true;
		}

		FValidateAssetsResults Results;
		EditorValidationSubsystem->ValidateAssetsWithSettings(bValidateDependencies ? DependentAssets.Array() : SelectedAssets, Settings, Results);
	}
}

void FDataValidationModule::ValidateFolders(const TArray<FString>& SelectedFolders, const EDataValidationUsecase InValidationUsecase)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Form a filter from the paths
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	for (const FString& Folder : SelectedFolders)
	{
		Filter.PackagePaths.Emplace(*Folder);
	}

	// Query for a list of assets in the selected paths
	TArray<FAssetData> AssetList;
	AssetRegistryModule.Get().GetAssets(Filter, AssetList);

	// UE-144978 : Remove ExternalActors & ExternalObjects from assets to be validated.
	// If ExternalActors are not loaded, they will spam the validation log as they can't
	// be loaded on the fly like other assets.
	auto IsAssetPackageExternal = [](const FAssetData& AssetData)
	{
		FString ObjectPath = AssetData.GetObjectPathString();
		FStringView ClassName, PackageName, ObjectName, SubObjectName;
		FPackageName::SplitFullObjectPath(FStringView(ObjectPath), ClassName, PackageName, ObjectName, SubObjectName);

		return FName(PackageName) != AssetData.PackageName;
	};
	AssetList.SetNum(Algo::RemoveIf(AssetList, IsAssetPackageExternal));

	ValidateAssets(AssetList, false, InValidationUsecase);
}

void FDataValidationModule::OnPackageSaved(const FString& PackageFileName, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
{
	UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>();
	if (EditorValidationSubsystem && Package)
	{
		EditorValidationSubsystem->ValidateSavedPackage(Package->GetFName(), ObjectSaveContext.IsProceduralSave());
	}
}

void FDataValidationModule::OnCookStarted(UE::Cook::ICookInfo& CookInfo)
{
	using namespace UE::Cook;

	if (DataValidationOptions::bReportCookValidationStats)
	{
		CookValidationStats.Emplace();
	}
	else
	{
		CookValidationStats.Reset();
	}

	if (EnumHasAnyFlags(CookInfo.GetCookValidationOptions(), ECookValidationOptions::RunAssetValidation))
	{
		// Log the enabled validators
		// This will also preload any BP-based validators before we start the main cook
		if (UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>())
		{
			int32 NumEnabledValidators = 0;

			UE_LOGF(LogContentValidation, Display, "Enabled validators:");
			EditorValidationSubsystem->ForEachEnabledValidator([&NumEnabledValidators](UEditorValidatorBase* Validator)
			{
				++NumEnabledValidators;
				UE_LOGF(LogContentValidation, Display, "\t%ls", *Validator->GetClass()->GetClassPathName().ToString());
				return true;
			});

			if (CookValidationStats)
			{
				CookValidationStats->ValidatorStats.Reserve(NumEnabledValidators);
			}
		}
	}
}

void FDataValidationModule::OnCookFinished(UE::Cook::ICookInfo& CookInfo)
{
	using namespace UE::Cook;

	if (CookValidationStats && EnumHasAnyFlags(CookInfo.GetCookValidationOptions(), ECookValidationOptions::RunAssetValidation))
	{
		// Report the validators, slowest to fastest
		CookValidationStats->ValidatorStats.ValueSort(
			[](const FValidatorStatistics& One, const FValidatorStatistics& Two)
			{
				return One.TotalDurationSeconds > Two.TotalDurationSeconds;
			});

		UE_LOGF(LogContentValidation, Display, "Validation stats: (%d external objects took %f seconds to load and %f seconds to unload)", CookValidationStats->ExternalObjectTotalCount, CookValidationStats->ExternalObjectLoadTotalDurationSeconds, CookValidationStats->ExternalObjectUnloadTotalDurationSeconds);
		for (const TTuple<FTopLevelAssetPath, FValidatorStatistics>& ValidatorStatsPair : CookValidationStats->ValidatorStats)
		{
			UE_LOGF(LogContentValidation, Display, "\t%ls validated %d assets in %f seconds", *ValidatorStatsPair.Key.ToString(), ValidatorStatsPair.Value.AssetsValidated, ValidatorStatsPair.Value.TotalDurationSeconds);
		}
	}
}

#if ENABLE_COOK_STATS
void FDataValidationModule::LogCookStats(FCookStatsManager::AddStatFuncRef AddStat)
{
	if (CookValidationStats)
	{
		UE::DataValidation::ValidationStatsToAnalyticEventData(CookValidationStats->ValidatorStats, AddStat);

		if (CookValidationStats->ExternalObjectTotalCount > 0)
		{
			AddStat(
				TEXT("Validation_ExternalObjectLoad"),
				FCookStatsManager::CreateKeyValueArray(TEXT("TotalDurationSecs"), CookValidationStats->ExternalObjectLoadTotalDurationSeconds, TEXT("AssetsValidated"), CookValidationStats->ExternalObjectTotalCount)
				);

			AddStat(
				TEXT("Validation_ExternalObjectUnload"),
				FCookStatsManager::CreateKeyValueArray(TEXT("TotalDurationSecs"), CookValidationStats->ExternalObjectUnloadTotalDurationSeconds, TEXT("AssetsValidated"), CookValidationStats->ExternalObjectTotalCount)
				);
		}
	}
}
#endif

EDataValidationResult FDataValidationModule::OnValidateSourcePackageDuringCook(UPackage* Package, FDataValidationContext& ValidationContext)
{
	EDataValidationResult FinalValidationResult = EDataValidationResult::NotValidated;

	if (UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>())
	{
		TArray<FAssetData> AssetList;
		IAssetRegistry::GetChecked().GetAssetsByPackageName(Package->GetFName(), AssetList, /*bIncludeOnlyOnDiskAssets*/true);

		{
			FValidateAssetsSettings Settings;
			Settings.ValidationUsecase = ValidationContext.GetValidationUsecase();
			AssetList.RemoveAll([EditorValidationSubsystem, &Settings, &ValidationContext](const FAssetData& AssetData)
			{
				return !EditorValidationSubsystem->ShouldValidateAsset(AssetData, Settings, ValidationContext);
			});
		}

		if (AssetList.Num() > 0)
		{
			// Broadcast the Editor event before we start validating. This lets other systems (such as Sequencer) restore the state
			// of the level to what is actually saved on disk before performing validation.
			if (FEditorDelegates::OnPreAssetValidation.IsBound())
			{
				FEditorDelegates::OnPreAssetValidation.Broadcast();
			}

			FMessageLog DataValidationLog(UE::DataValidation::MessageLogName);
			for (const FAssetData& AssetData : AssetList)
			{
				if (UObject* Asset = AssetData.FastGetAsset(/*bLoad*/false))
				{
					DataValidationLog.Info()
						->AddToken(FAssetDataToken::Create(AssetData))
						->AddToken(FTextToken::Create(LOCTEXT("Data.ValidatingAsset", "Validating asset")));

					TOptional<UE::DataValidation::FScopedLoadExternalObjects> ExternalObjectsLoader;
					if (!ValidationContext.GetAssociatedExternalObjects().IsEmpty())
					{
						if (CookValidationStats)
						{
							double LoadDuration = 0.0;
							{
								FScopedDurationTimer ValidationTimer(LoadDuration);
								ExternalObjectsLoader.Emplace(Asset, ValidationContext, true);
							}
							CookValidationStats->ExternalObjectLoadTotalDurationSeconds += LoadDuration;
							CookValidationStats->ExternalObjectTotalCount += ValidationContext.GetAssociatedExternalObjects().Num();
						}
						else
						{
							ExternalObjectsLoader.Emplace(Asset, ValidationContext, true);
						}
					}

					UE::DataValidation::FScopedLogMessageGatherer LogGatherer;
					EDataValidationResult ValidationResult = EditorValidationSubsystem->ValidateObjectInternal(AssetData, Asset, ValidationContext, CookValidationStats ? &CookValidationStats->ValidatorStats : nullptr);

					TArray<FString> LogWarnings;
					TArray<FString> LogErrors;
					LogGatherer.Stop(LogWarnings, LogErrors);

					if (LogWarnings.Num() > 0)
					{
						TStringBuilder<2048> Buffer;
						Buffer.Join(LogWarnings, LINE_TERMINATOR);
						ValidationContext.AddMessage(EMessageSeverity::Warning)
							->AddToken(FAssetDataToken::Create(AssetData))
							->AddText(LOCTEXT("DataValidation.DuringValidationWarnings", "Warnings logged while validating asset {0}"), FText::FromStringView(Buffer.ToView()));
					}
					if (LogErrors.Num() > 0)
					{
						TStringBuilder<2048> Buffer;
						Buffer.Join(LogErrors, LINE_TERMINATOR);
						ValidationContext.AddMessage(EMessageSeverity::Error)
							->AddToken(FAssetDataToken::Create(AssetData))
							->AddText(LOCTEXT("DataValidation.DuringValidationErrors", "Errors logged while validating asset {0}"), FText::FromStringView(Buffer.ToView()));
						ValidationResult = EDataValidationResult::Invalid;
					}

					FinalValidationResult = CombineDataValidationResults(FinalValidationResult, ValidationResult);

					UE::DataValidation::AddAssetValidationMessages(AssetData, DataValidationLog, ValidationContext);
					DataValidationLog.Flush();

					if (ExternalObjectsLoader)
					{
						if (CookValidationStats)
						{
							double UnloadDuration = 0.0;
							{
								FScopedDurationTimer ValidationTimer(UnloadDuration);
								ExternalObjectsLoader.Reset();
							}
							CookValidationStats->ExternalObjectUnloadTotalDurationSeconds += UnloadDuration;
						}
						else
						{
							ExternalObjectsLoader.Reset();
						}
					}
				}
			}

			// Broadcast now that we're complete so other systems can go back to their previous state.
			if (FEditorDelegates::OnPostAssetValidation.IsBound())
			{
				FEditorDelegates::OnPostAssetValidation.Broadcast();
			}
		}
	}

	return FinalValidationResult;
}

#undef LOCTEXT_NAMESPACE
