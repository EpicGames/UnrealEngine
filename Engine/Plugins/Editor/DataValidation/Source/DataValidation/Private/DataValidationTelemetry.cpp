// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataValidationTelemetry.h"

#include "CoreGlobals.h"
#include "EditorValidatorSubsystem.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::Telemetry::DataValidation
{

FValidateAssets::FValidateAssets(const FValidateAssetsSettings& InSettings)
{
	Mode = IsRunningCommandlet() ? EValidationMode::Commandlet : EValidationMode::Editor;

	bSkipExcludedDirectories = InSettings.bSkipExcludedDirectories;
	ValidationUsecase = InSettings.ValidationUsecase;
	bLoadAssetsForValidation = InSettings.bLoadAssetsForValidation;
	bUnloadAssetsLoadedForValidation = InSettings.bUnloadAssetsLoadedForValidation;
	bLoadExternalObjectsForValidation = InSettings.bLoadExternalObjectsForValidation;
	bCaptureAssetLoadLogs = InSettings.bCaptureAssetLoadLogs;
	bCaptureLogsDuringValidation = InSettings.bCaptureLogsDuringValidation;
	bCaptureWarningsDuringValidationAsErrors = InSettings.bCaptureWarningsDuringValidationAsErrors;
	MaxAssetsToValidate = InSettings.MaxAssetsToValidate;
	bValidateReferencersOfDeletedAssets = InSettings.bValidateReferencersOfDeletedAssets;
}

void FValidateAssets::SetResults(EDataValidationResult InOverallResult, const FValidateAssetsResults& InResults)
{
	Result = InOverallResult;
	NumRequested = InResults.NumRequested;
	NumExternalObjects = InResults.NumExternalObjects;
	NumChecked = InResults.NumChecked;
	NumValid = InResults.NumValid;
	NumInvalid = InResults.NumInvalid;
	NumSkipped = InResults.NumSkipped;
	NumWarnings = InResults.NumWarnings;
	NumUnableToValidate = InResults.NumUnableToValidate;
	bAssetLimitReached = InResults.bAssetLimitReached;
	for (const TPair<FTopLevelAssetPath, ::FValidatorStatistics>& Pair : InResults.ValidatorStatistics)
	{
		FValidatorStatistics& Stats = ValidatorStatistics.FindOrAdd(Pair.Key);
		Stats.AssetsValidated += Pair.Value.AssetsValidated;
		Stats.AssetsAddedForValidation += Pair.Value.AssetsAddedForValidation;
	}
}

}