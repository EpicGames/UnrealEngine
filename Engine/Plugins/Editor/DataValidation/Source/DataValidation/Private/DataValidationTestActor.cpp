// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataValidationTestActor.h"

#include "AssetRegistry/AssetData.h"
#include "Components/BillboardComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Texture2D.h"
#include "Misc/DataValidation.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataValidationTestActor)

#define LOCTEXT_NAMESPACE "AssetValidation"

ADataValidationTestActor::ADataValidationTestActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SpriteComponent = CreateDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	RootComponent = SpriteComponent;

	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> SpriteTexture;
		FName ID_Info;
		FText NAME_Info;
		FConstructorStatics()
			: SpriteTexture(TEXT("/Engine/EditorResources/S_Actor"))
			, ID_Info(TEXT("Info"))
			, NAME_Info(NSLOCTEXT("SpriteCategory", "Info", "Info"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SpriteComponent->Sprite = ConstructorStatics.SpriteTexture.Get();
	SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Info;
	SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Info;
	SpriteComponent->bIsScreenSizeScaled = true;

	bIsSpatiallyLoaded = true;
}

EDataValidationResult ADataValidationTestActor::IsDataValid(FDataValidationContext& Context) const 
{
    if (!bPassValidation)
    {
        Context.AddMessage(FAssetData(this), EMessageSeverity::Error, LOCTEXT("bPassValidationFalse", "bPassValidation is false"));
        return EDataValidationResult::Invalid;
    }
    return EDataValidationResult::Valid;
}

#undef LOCTEXT_NAMESPACE 
