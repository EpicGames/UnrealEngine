// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "DataValidationTestActor.generated.h"

class UBillboardComponent;

UCLASS(showcategories=(HLOD, WorldPartition, DataLayers, Transformation))
class ADataValidationTestActor : public AActor
{
    GENERATED_BODY()

public:
	ADataValidationTestActor(const FObjectInitializer& ObjectInitializer);

    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Validation")
    bool bPassValidation = false;

#if WITH_EDITOR
    EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
#endif

	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
};