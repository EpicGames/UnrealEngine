// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirtyFilesChangelistValidator.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataToken.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DataValidationChangelist.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h"
#include "Misc/DataValidation.h"
#include "Misc/Paths.h"
#include "Containers/Map.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DirtyFilesChangelistValidator)

#define LOCTEXT_NAMESPACE "DirtyFilesChangelistValidation"

bool UDirtyFilesChangelistValidator::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	if (InContext.GetValidationUsecase() == EDataValidationUsecase::Commandlet)
	{
		return false;
	}
	return (InAsset != nullptr) && (UDataValidationChangelist::StaticClass() == InAsset->GetClass());
}

EDataValidationResult UDirtyFilesChangelistValidator::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	UDataValidationChangelist* DataValidationChangelist = CastChecked<UDataValidationChangelist>(InAsset);

	// Retrieve current unsaved packages
	TArray<UPackage*> DirtyPackages;
	FEditorFileUtils::GetDirtyPackages(DirtyPackages, FEditorFileUtils::FShouldIgnorePackage::Default);

	auto Predicate = [](const UPackage* InPackage) -> bool
	{
		if (InPackage == nullptr)
		{
			return false;
		}
		
		const FString LocalFullPath(InPackage->GetLoadedPath().GetLocalFullPath());

		if (LocalFullPath.IsEmpty())
		{
			return false;
		}

		return true;
	};

	TSet<FName> DirtyPackagesSet;
	Algo::TransformIf(DirtyPackages, DirtyPackagesSet, Predicate, UE_PROJECTION_MEMBER(UPackage, GetFName));

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
    IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Check if any of changelist's files is unsaved
	for (FName PackageName : DataValidationChangelist->ModifiedPackageNames)
	{
		if (DirtyPackagesSet.Contains(PackageName))
		{
			FText CurrentError = LOCTEXT("DirtyFilesFound.Changelist.Error", "This changelist contains an unsaved asset. Please save to proceed.");
			TSharedRef<FTokenizedMessage> Message = AssetMessage(EMessageSeverity::Error, CurrentError);
			TArray<FAssetData> PackageAssets;
			if (AssetRegistry.GetAssetsByPackageName(PackageName, PackageAssets, true))
			{
				Message->AddToken(FAssetDataToken::Create(PackageAssets[0]));
			}
			else
			{
				Message->AddText(FText::FromName(PackageName));
			}
		}
	}

	if (GetValidationResult() != EDataValidationResult::Invalid)
	{
		AssetPasses(InAsset);
	}

	return GetValidationResult();
}

#undef LOCTEXT_NAMESPACE

