// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorValidatorBase.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataToken.h"
#include "EditorToolDelegates.h"
#include "EditorValidatorSubsystem.h"
#include "Misc/DataValidation.h"
#include "Trace/Trace.inl"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorValidatorBase)

#define LOCTEXT_NAMESPACE "AssetValidation"

EDataValidationResult UEditorValidatorBase::ValidateLoadedAsset(const FAssetData& AssetData, UObject* Asset, FDataValidationContext& Context)
{
	EDataValidationResult Result = EDataValidationResult::NotValidated;
	
	ResetValidationState();
	FDateTime ValidationTime = FDateTime::Now();
	bool bTriedToValidate = false;
	
	TGuardValue<TObjectPtr<UObject>> ObjectGuard(CurrentObjectBeingValidated, Asset);
	TGuardValue<const FAssetData*> AssetGuard(CurrentAssetBeingValidated, &AssetData);

	if (K2_CanValidate(Context.GetValidationUsecase()) && K2_CanValidateAsset(Asset))
	{
		EDataValidationResult K2Result = K2_ValidateLoadedAsset(Asset);
		K2Result = CombineDataValidationResults(GetValidationResult(), K2Result);
		ensureMsgf(K2Result != EDataValidationResult::NotValidated, TEXT("Validator %s did not return a validation result from BP validation path for asset %s"), *GetClass()->GetPathName(), *Asset->GetPathName());
		
		Result = CombineDataValidationResults(Result, K2Result);
		bTriedToValidate = true;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (CanValidate_Implementation(Context.GetValidationUsecase()) && CanValidateAsset_Implementation(Asset))
	{
		ensureMsgf(false, TEXT("Validator %s running deprecated validation path. This validator should implement ValidateLoadedAsset_Implementation(UObject*, FDataValidationContext&) instead"),
			*GetClass()->GetPathName());
		
		EDataValidationResult NewResult = ValidateLoadedAsset_Implementation(Asset, AllErrors);
		NewResult = CombineDataValidationResults(GetValidationResult(), NewResult);
		ensureMsgf(NewResult != EDataValidationResult::NotValidated, TEXT("Validator %s did not return a validation result from legacy validation path for asset %s"), *GetClass()->GetPathName(), *Asset->GetPathName());

		Result = CombineDataValidationResults(Result, NewResult);
		bTriedToValidate = true;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	if (CanValidateAsset_Implementation(AssetData, Asset, Context))
	{
		EDataValidationResult NewResult = ValidateLoadedAsset_Implementation(AssetData, Asset, Context);
		UE_LOGF(LogContentValidation, VeryVerbose, "Validation result: '%d'", int(NewResult));

		NewResult = CombineDataValidationResults(GetValidationResult(), NewResult);
		ensureMsgf(NewResult != EDataValidationResult::NotValidated, TEXT("Validator %s did not return a validation result from native validation path for asset %s"), *GetClass()->GetPathName(), *Asset->GetPathName());

		Result = CombineDataValidationResults(Result, NewResult);
		bTriedToValidate = true;
	}
	
	if(bTriedToValidate && LogContentValidation.GetVerbosity() >= ELogVerbosity::VeryVerbose)
	{
		FDateTime CurrentTime = FDateTime::Now();
		FTimespan ElapsedTimeSpan = (CurrentTime - ValidationTime);
		float ElapsedTimeMS = ElapsedTimeSpan.GetTotalMilliseconds();
		FNumberFormattingOptions TimeFormat;
		TimeFormat.MinimumFractionalDigits = 5;
		FText ElapsedTimeMessage = FText::Format(LOCTEXT("ElapsedTime", "Checking {0} with {1} took {2} ms."), 
			FText::FromString(Asset->GetPathName()),
			FText::FromString(GetClass()->GetPathName()),
			FText::AsNumber(ElapsedTimeMS, &TimeFormat));
		UE_LOGF(LogContentValidation, VeryVerbose, "%ls", *ElapsedTimeMessage.ToString());
	}
	Result = CombineDataValidationResults(Result, ExtractValidationState(Context)); // Extract messages and validation state from members (primarily for use by BP interface functions)

	if (bTriedToValidate)
	{
		FEditorToolDelegates::OnEditorToolStarted.Broadcast(GetClass()->GetName());
	}

	return Result;
}

bool UEditorValidatorBase::K2_CanValidate_Implementation(const EDataValidationUsecase InUsecase) const
{
	return true;
}

bool UEditorValidatorBase::K2_CanValidateAsset_Implementation(UObject* InAsset) const
{
	return false;
}

EDataValidationResult UEditorValidatorBase::K2_ValidateLoadedAsset_Implementation(UObject* InAsset)
{
	return EDataValidationResult::NotValidated;
}

void UEditorValidatorBase::AssetFails(const UObject* InAsset, const FText& InMessage)
{
	FText FailureMessage;
	if (bOnlyPrintCustomMessage)
	{
		FailureMessage = InMessage;
	}
	else
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("CustomMessage"), InMessage);
		Arguments.Add(TEXT("ValidatorName"), FText::FromString(GetClass()->GetName()));
		FailureMessage = FText::Format(LOCTEXT("AssetCheck_Message_Error", "{CustomMessage}. ({ValidatorName})"), Arguments);
	}


	AllErrors.Add(FailureMessage);
	ValidationResult = EDataValidationResult::Invalid;
}

void UEditorValidatorBase::AssetWarning(const UObject* InAsset, const FText& InMessage)
{
	FText WarningMessage;
	if (bOnlyPrintCustomMessage)
	{
		WarningMessage = InMessage;
	}
	else
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("CustomMessage"), InMessage);
		Arguments.Add(TEXT("ValidatorName"), FText::FromString(GetClass()->GetName()));
		WarningMessage = FText::Format(LOCTEXT("AssetCheck_Message_Warning", "{CustomMessage}. ({ValidatorName})"), Arguments);
	}

	AllWarnings.Add(WarningMessage);
}

void UEditorValidatorBase::AssetPasses(const UObject* InAsset)
{
	if (LogContentValidation.GetVerbosity() >= ELogVerbosity::Verbose)
	{
		FFormatNamedArguments Arguments;
		if (InAsset)
		{
			Arguments.Add(TEXT("AssetName"), FText::FromName(InAsset->GetFName()));
		}
		Arguments.Add(TEXT("ValidatorName"), FText::FromString(GetClass()->GetPathName()));
	}

	ensureMsgf(ValidationResult != EDataValidationResult::Invalid, TEXT("%s: AssetPasses called after errors were reported"), *GetClass()->GetPathName());
	ValidationResult = EDataValidationResult::Valid;
}

TSharedRef<FTokenizedMessage> UEditorValidatorBase::AssetMessage(const FAssetData& InAssetData, EMessageSeverity::Type InSeverity, const FText& InText)
{
	if (InSeverity == EMessageSeverity::Error)
	{
		ValidationResult = EDataValidationResult::Invalid;
	}
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(InSeverity);
	if (InAssetData.IsValid())
	{
		Message->AddToken(FAssetDataToken::Create(InAssetData));		
	}
	if (!InText.IsEmpty())
	{
		Message->AddText(InText);
	}
	AllMessages.Add(Message);
	return Message; 
}

TSharedRef<FTokenizedMessage> UEditorValidatorBase::AssetMessage(EMessageSeverity::Type InSeverity, const FText& InText)
{
	return AssetMessage(FAssetData{}, InSeverity, InText);
}

void UEditorValidatorBase::AddLegacyValidationErrors(TArray<FText> InErrors)
{
	AllErrors.Append(MoveTemp(InErrors));
}

void UEditorValidatorBase::ResetValidationState()
{
	ValidationResult = EDataValidationResult::NotValidated;
	AllMessages.Empty();
	AllWarnings.Empty();
	AllErrors.Empty();
}

EDataValidationResult UEditorValidatorBase::ExtractValidationState(FDataValidationContext& InOutContext) const
{
	for (const FText& Warning : AllWarnings)
	{
		InOutContext.AddWarning(Warning);	
	}
	for (const FText& Error : AllErrors)
	{
		InOutContext.AddError(Error);	
	}
	for (const TSharedRef<FTokenizedMessage>& Message : AllMessages)
	{
		InOutContext.AddMessage(Message);	
	}
	return ValidationResult;
}

const TArray<FText>& UEditorValidatorBase::GetAllWarnings() const
{
	return AllWarnings;
}

#undef LOCTEXT_NAMESPACE

