// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorValidatorHelpers.h"

#include "Algo/AllOf.h"
#include "AnalyticsEventAttribute.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataToken.h"
#include "Containers/ArrayView.h"
#include "EditorValidatorSubsystem.h"
#include "JsonUtils/RapidJsonUtils.h"
#include "Logging/MessageLog.h"
#include "Misc/DataValidation.h"
#include "Misc/WildcardString.h"

namespace UE::DataValidation
{

void AddAssetValidationMessages(
    FMessageLog& Log,
    const FDataValidationContext& ValidationContext)
{
    for (const FDataValidationContext::FIssue& Issue : ValidationContext.GetIssues())
    {
        if (Issue.TokenizedMessage.IsValid())
        {
            Log.AddMessage(Issue.TokenizedMessage.ToSharedRef());
        }
        else
        {
            Log.Message(Issue.Severity, Issue.Message);
        }
    }
}

void AddAssetValidationMessages(
    const FAssetData& ForAsset,
    FMessageLog& Log,
    const FDataValidationContext& ValidationContext)
{
    for (const FDataValidationContext::FIssue& Issue : ValidationContext.GetIssues())
    {
        if (Issue.TokenizedMessage.IsValid())
        {
            Log.AddMessage(Issue.TokenizedMessage.ToSharedRef());
        }
        else
        {
            AddAssetValidationMessages(ForAsset, Log, Issue.Severity, { Issue.Message });
        }
    }
}

void AddAssetValidationMessages(
    const FAssetData& ForAsset,
    FMessageLog& Log,
    EMessageSeverity::Type Severity,
    TConstArrayView<FText> Messages)
{
    const FString PackageNameString = ForAsset.PackageName.ToString();
    for (const FText& Msg : Messages)
    {
        const FString AssetLogString = FAssetMsg::GetAssetLogString(*PackageNameString, Msg.ToString());
        FString BeforeAsset;
        FString AfterAsset;
        TSharedRef<FTokenizedMessage> TokenizedMessage = Log.Message(Severity);
        if (AssetLogString.Split(PackageNameString, &BeforeAsset, &AfterAsset))
        {
            if (!BeforeAsset.IsEmpty())
            {
                TokenizedMessage->AddToken(FTextToken::Create(FText::FromString(BeforeAsset)));
            }
            TokenizedMessage->AddToken(FAssetDataToken::Create(ForAsset));
            if (!AfterAsset.IsEmpty())
            {
                TokenizedMessage->AddToken(FTextToken::Create(FText::FromString(AfterAsset)));
            }
        }
        else
        {
            TokenizedMessage->AddToken(FTextToken::Create(FText::FromString(AssetLogString)));
        }
    }
}

FText MakeTimestampedMessageLogPageTitle(const FText& BaseTitle)
{
	return MakeTimestampedMessageLogPageTitle(BaseTitle, FDateTime::UtcNow());
}

FText MakeTimestampedMessageLogPageTitle(const FText& BaseTitle, const FDateTime& UtcTimestamp)
{
	static const FTextFormat TimestampedPageTitleFmt = NSLOCTEXT("DataValidation", "TimestampedPageTitleFmt", "[{0}]\t{1}");
	return FText::Format(TimestampedPageTitleFmt, FText::AsDateTime(UtcTimestamp, EDateTimeStyle::Short, EDateTimeStyle::Default), BaseTitle);
}

FString ValidationStatsToJsonString(const TMap<FTopLevelAssetPath, FValidatorStatistics>& ValidationStats)
{
	UE::Json::FDocument JsonDocument(rapidjson::kArrayType);
	{
		UE::Json::FArray JsonRootArray = JsonDocument.GetArray();
		UE::Json::FAllocator& JsonAllocator = JsonDocument.GetAllocator();

		for (const TTuple<FTopLevelAssetPath, FValidatorStatistics>& ValidationStatsPair : ValidationStats)
		{
			UE::Json::FValue ValidationStatsJsonObject(rapidjson::kObjectType);
			{
				FString ValidatorType = ValidationStatsPair.Key.GetAssetName().ToString();
				ValidatorType.ToLowerInline();

				ValidationStatsJsonObject.AddMember(
					UE::Json::MakeStringRef(TEXTVIEW("validator_type")),
					UE::Json::MakeStringValue(ValidatorType, JsonAllocator),
					JsonAllocator);

				ValidationStatsJsonObject.AddMember(
					UE::Json::MakeStringRef(TEXTVIEW("total_duration_secs")),
					ValidationStatsPair.Value.TotalDurationSeconds,
					JsonAllocator);

				ValidationStatsJsonObject.AddMember(
					UE::Json::MakeStringRef(TEXTVIEW("assets_validated")),
					ValidationStatsPair.Value.AssetsValidated,
					JsonAllocator);
			}
			JsonRootArray.PushBack(MoveTemp(ValidationStatsJsonObject), JsonAllocator);
		}
	}

	return UE::Json::WriteCompact(JsonDocument);
}

void ValidationStatsToAnalyticEventData(const TMap<FTopLevelAssetPath, FValidatorStatistics>& ValidationStats, TArray<FAnalyticsEventAttribute>& OutEventAttributes)
{
	// This matches the format used by FEditorTelemetry::RecordEvent_Cooking, where StatName ("Validation") is combined with each AttrName ("Stats") to form "{StatName}_{AttrName}"
	// (see the below variant of ValidationStatsToAnalyticEventData)
	OutEventAttributes.Emplace(TEXT("Validation_Stats"), FJsonFragment(ValidationStatsToJsonString(ValidationStats)));
}

#if ENABLE_COOK_STATS
void ValidationStatsToAnalyticEventData(const TMap<FTopLevelAssetPath, FValidatorStatistics>& ValidationStats, FCookStatsManager::AddStatFuncRef AddStat)
{
	// StatName ("Validation") is combined with each AttrName ("Stats") in FEditorTelemetry::RecordEvent_Cooking, to form "{StatName}_{AttrName}"
	// (see the above variant of ValidationStatsToAnalyticEventData)
	AddStat(TEXT("Validation"), FCookStatsManager::CreateKeyValueArray(TEXT("Stats"), ValidationStatsToJsonString(ValidationStats)));
}
#endif

struct FLogMessageGathererImpl : public FOutputDevice
{
    FLogMessageGathererImpl()
    {
        TDelegate<TArray<FName>(void)>& DefaultIgnoreCategoriesDelegate = FScopedLogMessageGatherer::GetDefaultIgnoreCategoriesDelegate();
        if (DefaultIgnoreCategoriesDelegate.IsBound())
        {
            DefaultIgnoreCategories = DefaultIgnoreCategoriesDelegate.Execute();
        }
        TDelegate<TArray<FWildcardString>(void)>& DefaultIgnorePatternsDelegate = FScopedLogMessageGatherer::GetDefaultIgnorePatternsDelegate();
        if (DefaultIgnorePatternsDelegate.IsBound())
        {
            DefaultIgnorePatterns = DefaultIgnorePatternsDelegate.Execute();
        }

		IgnoreCategories = DefaultIgnoreCategories;
		IgnorePatterns = DefaultIgnorePatterns;

        GLog->AddOutputDevice(this);
    }
    
    ~FLogMessageGathererImpl()
    {
        // Remove this as an output device, internally this handles synchronization with Serialize for us so we don't need to protect against Serialize and ~FLogMessageGathererImpl running concurrently 
        // we deliberately don't lock the Critical Section (CS), as this would invert the locking order between FLogMessageGathererImpl & FOutputDeviceRedirector leading to potential a deadlock 
        GLog->RemoveOutputDevice(this);
    }
   
    TTuple<int32, int32> AddIgnoreCategories(TConstArrayView<FName> NewCategories)
    {
        FScopeLock _(&CS);    
        TTuple<int32, int32> Handle{IgnoreCategories.Num(), IgnoreCategories.Num() + NewCategories.Num()};
        IgnoreCategories.Append(NewCategories);
        return Handle;
    }

    void RemoveIgnoreCategories(TTuple<int32, int32> Range) 
    {
        FScopeLock _(&CS);    
        for (int32 i=Range.Key; i < Range.Value; ++i)
        {
            IgnoreCategories[i] = NAME_None;
        }
        if (Algo::AllOf(IgnoreCategories, UE_PROJECTION_MEMBER(FName, IsNone)))
        {
            IgnoreCategories.Reset();
        }
    }

    TTuple<int32, int32> AddIgnorePatterns(TConstArrayView<FWildcardString> NewPatterns)
    {
        FScopeLock _(&CS);
        TTuple<int32, int32> Handle{IgnorePatterns.Num(), IgnorePatterns.Num() + NewPatterns.Num()};
        IgnorePatterns.Append(NewPatterns);
        return Handle;
    }

    void RemoveIgnorePatterns(TTuple<int32, int32> Range)
    {
        FScopeLock _(&CS);    
        for (int32 i=Range.Key; i < Range.Value; ++i)
        {
            IgnorePatterns[i] = FWildcardString{};
        }
        if (Algo::AllOf(IgnorePatterns, UE_PROJECTION_MEMBER(FWildcardString, IsEmpty)))
        {
            IgnorePatterns.Reset();
        }
    }

    virtual bool CanBeUsedOnMultipleThreads() const override
    {
        return true;
    }

    virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
    {
        if (Verbosity > ELogVerbosity::Warning)
        {
            return;
        }

        FScopeLock _(&CS);

        if (!Category.IsNone() && IgnoreCategories.Contains(Category))
        {
            return;
        }
        if (IgnorePatterns.ContainsByPredicate([&V](const FWildcardString& S) { return S.IsEmpty() || S.IsMatch(V); }))
        {
            return;
        }
        
        if (Verbosity == ELogVerbosity::Warning)
        {
            Warnings.Emplace(V);
        }
        else
        {
            Errors.Emplace(V);
        }
    }
    
    void Stop(TArray<FString>& OutWarnings, TArray<FString>& OutErrors)
    {
        FScopeLock _(&CS);
        OutWarnings = MoveTemp(Warnings);
        OutErrors = MoveTemp(Errors);
    }

    FCriticalSection CS;
    
    TArray<FName> DefaultIgnoreCategories;
    TArray<FWildcardString> DefaultIgnorePatterns;
    
    TArray<FName> IgnoreCategories;
    TArray<FWildcardString> IgnorePatterns;
    
    TArray<FString> Errors;
    TArray<FString> Warnings;
};

thread_local FScopedLogMessageGatherer* CurrentGatherer = nullptr;

FScopedLogMessageGatherer::FScopedLogMessageGatherer(bool bEnabled)
{
    if (bEnabled)
    {
        Impl = new FLogMessageGathererImpl;
        Previous = CurrentGatherer;
        CurrentGatherer = this;
    }
}

FScopedLogMessageGatherer::~FScopedLogMessageGatherer()
{
    if (Impl)
    {
        CurrentGatherer = Previous;
        delete Impl;
    }
}

void FScopedLogMessageGatherer::Stop(TArray<FString>& OutWarnings, TArray<FString>& OutErrors)
{
    if (Impl)
    {
        Impl->Stop(OutWarnings, OutErrors);
    }
}

FScopedLogMessageGatherer* FScopedLogMessageGatherer::GetCurrentThreadGatherer()
{
    return CurrentGatherer;
}

TTuple<int32, int32> FScopedLogMessageGatherer::AddIgnoreCategories(TConstArrayView<FName> NewCategories)
{
    return Impl->AddIgnoreCategories(NewCategories);
}

void FScopedLogMessageGatherer::RemoveIgnoreCategories(TTuple<int32, int32> Range)
{
    Impl->RemoveIgnoreCategories(Range);
}

TTuple<int32, int32> FScopedLogMessageGatherer::AddIgnorePatterns(TConstArrayView<FWildcardString> NewPatterns)
{
    return Impl->AddIgnorePatterns(NewPatterns);
}

void FScopedLogMessageGatherer::RemoveIgnorePatterns(TTuple<int32, int32> Range)
{
    Impl->RemoveIgnorePatterns(Range);
}

TDelegate<TArray<FName>(void)>& FScopedLogMessageGatherer::GetDefaultIgnoreCategoriesDelegate()
{
    static TDelegate<TArray<FName>(void)> Delegate;
    return Delegate;
}

TDelegate<TArray<FWildcardString>(void)>& FScopedLogMessageGatherer::GetDefaultIgnorePatternsDelegate()
{
    static TDelegate<TArray<FWildcardString>(void)> Delegate;
    return Delegate;
}

FScopedIgnoreLogMessages::FScopedIgnoreLogMessages(TConstArrayView<FWildcardString> Patterns, TConstArrayView<FName> Categories)
{
    if(FScopedLogMessageGatherer* Gatherer = FScopedLogMessageGatherer::GetCurrentThreadGatherer())
    {
        CategoriesRange = Gatherer->AddIgnoreCategories(Categories);
        PatternsRange = Gatherer->AddIgnorePatterns(Patterns);
    }
}

FScopedIgnoreLogMessages::~FScopedIgnoreLogMessages()
{
    if(FScopedLogMessageGatherer* Gatherer = FScopedLogMessageGatherer::GetCurrentThreadGatherer())
    {
        Gatherer->RemoveIgnoreCategories(CategoriesRange);
        Gatherer->RemoveIgnorePatterns(PatternsRange);
    }
}

} // namespace UE::DataValidation
