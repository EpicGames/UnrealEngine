// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorValidatorSubsystem.h"

#include "Algo/AnyOf.h"
#include "Algo/RemoveIf.h"
#include "AssetCompilingManager.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetDataToken.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "CoreGlobals.h"
#include "DataValidationChangelist.h"
#include "DataValidationTelemetry.h"
#include "EditorValidatorHelpers.h"
#include "DirectoryWatcherModule.h"
#include "Editor.h"
#include "EditorClassUtils.h"
#include "EditorUtilityBlueprint.h"
#include "EditorValidatorBase.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Level.h"
#include "IAssetTools.h"
#include "IDirectoryWatcher.h"
#include "ISourceControlModule.h"
#include "LoadExternalObjectsForValidation.h"
#include "Logging/MessageLog.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/DataValidation.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopeExit.h"
#include "PackageTools.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "TelemetryRouter.h"
#include "UObject/ICookInfo.h"
#include "UObject/TopLevelAssetPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorValidatorSubsystem)

#define LOCTEXT_NAMESPACE "EditorValidationSubsystem"

DEFINE_LOG_CATEGORY(LogContentValidation);

namespace UE::DataValidation
{
	static int32 AssetValidationBatchSize = 50;
	static FAutoConsoleVariableRef CVarAssetValidationBatchSize(
		TEXT("EditorValidator.AssetValidationBatchSize"),
		AssetValidationBatchSize,
		TEXT("Controls number of assets per batch. Affects number of assets preloaded at a time, and number of assets unloaded at a time"));

	static bool bEnablePreloadingAssets = false;
	static FAutoConsoleVariableRef CVarEnablePreloadingAssets(
		TEXT("EditorValidator.EnablePreloadingAssets"),
		bEnablePreloadingAssets,
		TEXT("Enables preloading assets during validation to improve performance"));

	static int32 MaxAssetBatchPreloads = 20;
	static FAutoConsoleVariableRef CVarMaxAssetBatchPreloads(
		TEXT("EditorValidator.MaxAssetBatchPreloads"),
		MaxAssetBatchPreloads,
		TEXT("Number of asset batches to preload at a time"));

	void LogLoadingAsset(const FAssetData& AssetData, bool bUnloadAssetsLoadedForValidation)
	{
		UE_LOGFMT(LogContentValidation, Log, "Loading asset {Asset} for validation. After the validation we will {Action}unload it",
			("Asset", AssetData.ToSoftObjectPath()), ("Action", bUnloadAssetsLoadedForValidation ? TEXT("") : TEXT("not ")));
	};

	struct FAssetToValidate
	{
		const FAssetData* Data = nullptr;
		bool bWasAlreadyLoaded = false;
	};

	struct FValidationAssetRange
	{
		FValidationAssetRange(const TSet<FAssetData>& InAllAssets, const FValidateAssetsSettings& InSettings,
			TFunctionRef<bool(const FAssetData& Data)> InShouldSkipAsset, TFunction<void()> InOnAssetsLoaded)
			: OnAssetsLoaded(InOnAssetsLoaded)
			, BatchSize(FMath::Max(1, AssetValidationBatchSize))
			, bPreloadAssets(bEnablePreloadingAssets && !InSettings.bCaptureAssetLoadLogs)
		{
			Assets.Reserve(FMath::Clamp(InAllAssets.Num(), 0, InSettings.MaxAssetsToValidate));
			for (const FAssetData& Data : InAllAssets)
			{
				if (Assets.Num() >= InSettings.MaxAssetsToValidate)
				{
					break;
				}

				ensure(Data.IsValid());

				bool bAlreadyLoaded = Data.FastGetAsset(false) != nullptr;

				if (InShouldSkipAsset(Data) || (!InSettings.bLoadAssetsForValidation && !bAlreadyLoaded))
				{
					++NumSkipped;
					continue;
				}

				Assets.Add({ &Data, bAlreadyLoaded });
			}

			// Kick off the first set of preloads
			if (bPreloadAssets)
			{
				int32 NumBatches = FMath::CeilToInt32(static_cast<float>(Assets.Num()) / BatchSize);
				int32 InitialPreloads = FMath::Clamp(MaxAssetBatchPreloads, 1, NumBatches);
				for (int32 i = 0; i < InitialPreloads; ++i)
				{
					StartNextPreloadBatch();
				}
			}
		}

		~FValidationAssetRange()
		{
			// Flush any remaining async loads before unloading
			for (const TArray<int32>& Batch : BatchRequestIds)
			{
				FlushAsyncLoading(Batch);
			}

			UnloadTrackedPackages();
		}

		int32 GetNumSkipped() const { return NumSkipped; }

		void TrackPackageForUnload(UPackage* Package)
		{
			if (Package)
			{
				PackagesToUnload.Emplace(Package);
			}
		}

		struct FIterator
		{
			FValidationAssetRange& Owner;
			int32 Index = 0;

			FAssetToValidate operator*() const { return Owner.Assets[Index]; }

			FIterator& operator++()
			{
				++Index;
				Owner.OnAssetConsumed();
				return *this;
			}

			bool operator!=(const FIterator& Other) const { return Index != Other.Index; }
		};

		FIterator begin()
		{
			ensureMsgf(!bBeginCalled, TEXT("FValidationAssetRange does not support multiple iterations"));
			bBeginCalled = true;

			EnsureCurrentBatchReady();

			return { *this, 0 };
		}

		FIterator end() { return { *this, Assets.Num() }; }

	private:
		void UnloadTrackedPackages()
		{
			if (PackagesToUnload.IsEmpty())
			{
				return;
			}
				
			TArray<UPackage*> LocalPackages;
			CopyFromWeakArray(LocalPackages, PackagesToUnload);
			UPackageTools::UnloadPackages(LocalPackages);

			// Remove entries that were successfully unloaded
			// Keep entries still held alive by other preload handles and we'll try
			// again in the next batch boundary
			PackagesToUnload.RemoveAll([](const TWeakObjectPtr<UPackage>& Ptr) { return !Ptr.IsValid(); });
		}

		void StartNextPreloadBatch()
		{
			TArray<int32> RequestIds;
			RequestIds.Reserve(BatchSize);
			while (PrefetchIdx < Assets.Num() && RequestIds.Num() < BatchSize)
			{
				const FAssetToValidate& Asset = Assets[PrefetchIdx++];
				if (Asset.bWasAlreadyLoaded)
				{
					continue;
				}

				int32 RequestId = LoadPackageAsync(Asset.Data->PackageName.ToString());
				RequestIds.Add(RequestId);
			}

			if (RequestIds.IsEmpty())
			{
				return;
			}

			BatchRequestIds.Add(MoveTemp(RequestIds));
		}

		void EnsureCurrentBatchReady()
		{
			if (!bPreloadAssets || bWaitedForCurrentBatch)
			{
				return;
			}

			if (CurrentBatch < BatchRequestIds.Num())
			{
				FlushAsyncLoading(BatchRequestIds[CurrentBatch]);
				OnAssetsLoaded();
				bWaitedForCurrentBatch = true;
			}
		}

		void OnAssetConsumed()
		{
			if (++AssetsConsumedInBatch < BatchSize)
			{
				return;
			}

			if (bPreloadAssets)
			{
				++CurrentBatch;
				bWaitedForCurrentBatch = false;

				// Start the next batch in line
				StartNextPreloadBatch();
				EnsureCurrentBatchReady();
			}

			UnloadTrackedPackages();
			AssetsConsumedInBatch = 0;
		}

		TFunction<void()> OnAssetsLoaded;
		TArray<FAssetToValidate> Assets;
		TArray<TArray<int32>> BatchRequestIds;
		TArray<TWeakObjectPtr<UPackage>> PackagesToUnload;		
		int32 NumSkipped = 0;
		int32 BatchSize = 1;
		int32 PrefetchIdx = 0;
		int32 CurrentBatch = 0;
		int32 AssetsConsumedInBatch = 0;
		bool bPreloadAssets;
		bool bBeginCalled = false;
		bool bWaitedForCurrentBatch = false;
	};
} // namespace UE::DataValidation

FValidateAssetsSettings::FValidateAssetsSettings()
	: ShowMessageLogSeverity(EMessageSeverity::Warning)
	, MessageLogName(UE::DataValidation::MessageLogName)
	, MessageLogPageTitle(LOCTEXT("DataValidation.MessagePageTitle", "Data Validation"))
{
}

FValidateAssetsSettings::~FValidateAssetsSettings()
{
}

UDataValidationSettings::UDataValidationSettings()
{

}

FScopedDisableValidateOnSave::FScopedDisableValidateOnSave()
	: EditorValidationSubsystem(GEditor ? GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>() : nullptr)
{
	if (EditorValidationSubsystem)
	{
		EditorValidationSubsystem->PushDisableValidateOnSave();
	}
}

FScopedDisableValidateOnSave::~FScopedDisableValidateOnSave()
{
	if (EditorValidationSubsystem)
	{
		EditorValidationSubsystem->PopDisableValidateOnSave();
	}
}

UEditorValidatorSubsystem::UEditorValidatorSubsystem()
	: UEditorSubsystem()
{
	bAllowBlueprintValidators = true;
}

bool UEditorValidatorSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (GetClass() == UEditorValidatorSubsystem::StaticClass())
	{
		TArray<UClass*> ChildClasses;
		GetDerivedClasses(UEditorValidatorSubsystem::StaticClass(), ChildClasses, true);
		for (UClass* Child : ChildClasses)
		{
			if (Child->GetDefaultObject<UEditorSubsystem>()->ShouldCreateSubsystem(Outer))
			{
				// Do not create this class because one of our child classes wants to be created
				return false;
			}
		}
	}
	return true;
}

void UEditorValidatorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	// C++ registration
	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UEditorValidatorSubsystem::RegisterNativeValidators);

	// Blueprint registration
	if (IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		AssetRegistry.IsLoadingAssets())
	{
		// We are still discovering assets, listen for the completion delegate before building the validator list
		AssetRegistry.OnFilesLoaded().AddUObject(this, &UEditorValidatorSubsystem::RegisterBlueprintValidators);
	}
	else
	{
		RegisterBlueprintValidators();
	}

	// Register to SCC pre-submit callback
	ISourceControlModule::Get().RegisterPreSubmitChangelistValidationEx(FSourceControlPreSubmitChangelistValidationExDelegate::CreateUObject(this, &UEditorValidatorSubsystem::ValidateChangelistPreSubmit));
	ISourceControlModule::Get().RegisterPreSubmitFilesValidationEx(FSourceControlPreSubmitFilesValidationExDelegate::CreateUObject(this, &UEditorValidatorSubsystem::ValidateFilesPreSubmit));
}

bool UEditorValidatorSubsystem::ShouldValidateAsset(
	const FAssetData& 				Asset,
	const FValidateAssetsSettings& 	Settings,
	FDataValidationContext& 		InContext) const
{
	if (Asset.HasAnyPackageFlags(PKG_Cooked))
	{
		return false;
	}

	FNameBuilder AssetPackageNameBuilder(Asset.PackageName);
	FStringView AssetPackageNameView = AssetPackageNameBuilder.ToView();

	if (FPackageName::IsTempPackage(AssetPackageNameView))
	{
		return false;
	}

	if (FPackageName::IsVersePackage(AssetPackageNameView))
	{
		return false;
	}

	if (Settings.bSkipExcludedDirectories && IsPathExcludedFromValidation(AssetPackageNameView))
	{
		return false;
	}

	return true;
}

void UEditorValidatorSubsystem::RegisterNativeValidators()
{
	ensureAlwaysMsgf(!bHasRegisteredNativeValidators, TEXT("Native validators have already been registered!"));
	if (bHasRegisteredNativeValidators)
	{
		return;
	}

	TArray<UClass*> ValidatorClasses;
	GetDerivedClasses(UEditorValidatorBase::StaticClass(), ValidatorClasses);

	for (UClass* ValidatorClass : ValidatorClasses)
	{
		// GetDerivedClasses may include a mix of C++ and loaded BP classes
		// Skip any non-C++ classes, as well as anything that has already been registered by this point
		if (!ValidatorClass->HasAllClassFlags(CLASS_Abstract) && !Validators.Contains(ValidatorClass->GetClassPathName()) 
			&& FPackageName::IsScriptPackage(FNameBuilder(ValidatorClass->GetPackage()->GetFName())))
		{
			UEditorValidatorBase* Validator = NewObject<UEditorValidatorBase>(GetTransientPackage(), ValidatorClass);
			UE_LOGFMT(LogContentValidation, Log, "Adding native validator {ValidatorClass}", FTopLevelAssetPath(ValidatorClass));
			AddValidator_Internal(Validator);
		}
	}

	// Watch for native modules being added/removed
	FModuleManager::Get().OnModulesChanged().AddUObject(this, &UEditorValidatorSubsystem::OnNativeModulesChanged);

	bHasRegisteredNativeValidators = true;
}

void UEditorValidatorSubsystem::RegisterBlueprintValidators()
{
	ensureAlwaysMsgf(!bHasRegisteredBlueprintValidators, TEXT("Blueprint validators have already been registered!"));
	if (bHasRegisteredBlueprintValidators)
	{
		return;
	}

	if (bAllowBlueprintValidators)
	{
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

		// Locate all Blueprint-based validator classes (include unloaded)
		TSet<FTopLevelAssetPath> ValidatorClasses;
		AssetRegistry.GetDerivedClassNames({ UEditorValidatorBase::StaticClass()->GetClassPathName() }, {}, ValidatorClasses);

		for (const FTopLevelAssetPath& ValidatorClass : ValidatorClasses)
		{
			// IAssetRegistry::GetDerivedClassNames may include a mix of C++ and BP classes
			// Skip any C++ classes, as well as anything that has already been registered by this point
			if (!Validators.Contains(ValidatorClass) && !FPackageName::IsScriptPackage(FNameBuilder(ValidatorClass.GetPackageName())))
			{
				UE_LOGFMT(LogContentValidation, Log, "Adding blueprint validator {ValidatorClass}", FTopLevelAssetPath(ValidatorClass));
				AddValidator_Internal(ValidatorClass);
			}
		}

		// Watch for BPs being added/removed
		// Recompilation is handled by the standard reinstancing logic, as Validators is a UPROPERTY
		AssetRegistry.OnAssetsAdded().AddUObject(this, &UEditorValidatorSubsystem::OnAssetsAdded);
		AssetRegistry.OnAssetsRemoved().AddUObject(this, &UEditorValidatorSubsystem::OnAssetsRemoved);
	}

	bHasRegisteredBlueprintValidators = true;
}

void UEditorValidatorSubsystem::Deinitialize()
{
	CleanupValidators();

	// Unregister to SCC pre-submit callback
	ISourceControlModule::Get().UnregisterPreSubmitFilesValidation();
	ISourceControlModule::Get().UnregisterPreSubmitChangelistValidation();

	Super::Deinitialize();
}

void UEditorValidatorSubsystem::PushDisableValidateOnSave()
{
	checkf(DisableValidateOnSaveCount < std::numeric_limits<decltype(DisableValidateOnSaveCount)>::max(), TEXT("PushDisableValidateOnSave overflow!"));
	++DisableValidateOnSaveCount;
}

void UEditorValidatorSubsystem::PopDisableValidateOnSave()
{
	checkf(DisableValidateOnSaveCount > 0, TEXT("PopDisableValidateOnSave underflow!"));
	--DisableValidateOnSaveCount;
}

bool UEditorValidatorSubsystem::ShouldValidateOnSave(bool bProceduralSave) const
{
	// Skip if not enabled
	if (DisableValidateOnSaveCount > 0 || !GetDefault<UDataValidationSettings>()->bValidateOnSave)
	{
		return false;
	}

	// Skip auto and procedural saves
	// For performance reasons, don't validate when making a procedural save by default. Assumption is we validated when saving previously. 
	if (bProceduralSave || GEditor->IsAutosaving(EPackageAutoSaveType::Any))
	{
		return false;
	}

	return true;
}

void UEditorValidatorSubsystem::AddValidator(UEditorValidatorBase* InValidator)
{
	if (InValidator)
	{
		UE_LOGFMT(LogContentValidation, Log, "Explicitly adding custom validator {ValidatorClass}", FTopLevelAssetPath(InValidator->GetClass()));
		AddValidator_Internal(InValidator);
	}
}

void UEditorValidatorSubsystem::AddValidator(const FTopLevelAssetPath& InValidatorClass)
{
	UE_LOGFMT(LogContentValidation, Log, "Explicitly adding custom validator {ValidatorClass}", InValidatorClass);
	AddValidator_Internal(InValidatorClass);
}

void UEditorValidatorSubsystem::RemoveValidator(UEditorValidatorBase* InValidator)
{
	if (InValidator)
	{
		UE_LOGFMT(LogContentValidation, Log, "Explicitly removing custom validator {ValidatorClass}", FTopLevelAssetPath(InValidator->GetClass()));
		RemoveValidator_Internal(InValidator);
	}
}

void UEditorValidatorSubsystem::RemoveValidator(const FTopLevelAssetPath& InValidatorClass)
{
	UE_LOGFMT(LogContentValidation, Log, "Explicitly removing custom validator {ValidatorClass}", InValidatorClass);
	RemoveValidator_Internal(InValidatorClass);
}

void UEditorValidatorSubsystem::AddValidator_Internal(UEditorValidatorBase* InValidator)
{
	if (InValidator)
	{
		const FTopLevelAssetPath ValidatorClass = InValidator->GetClass()->GetClassPathName();
		Validators.Add(ValidatorClass, InValidator);
		ValidatorClassesPendingLoad.Remove(ValidatorClass);
	}
}

void UEditorValidatorSubsystem::AddValidator_Internal(FTopLevelAssetPath InValidatorClass)
{
	if (InValidatorClass.IsValid())
	{
		Validators.Add(InValidatorClass, nullptr);
		ValidatorClassesPendingLoad.Add(InValidatorClass);
	}
}

void UEditorValidatorSubsystem::RemoveValidator_Internal(UEditorValidatorBase* InValidator)
{
	if (InValidator)
	{
		RemoveValidator(InValidator->GetClass()->GetClassPathName());
	}
}

void UEditorValidatorSubsystem::RemoveValidator_Internal(FTopLevelAssetPath InValidatorClass)
{
	if (InValidatorClass.IsValid())
	{
		Validators.Remove(InValidatorClass);
		ValidatorClassesPendingLoad.Remove(InValidatorClass);
	}
}

void UEditorValidatorSubsystem::CleanupValidators()
{
	Validators.Empty();
	ValidatorClassesPendingLoad.Empty();
	NativeModulesPendingLoad.Empty();
	NativeModulesPendingUnload.Empty();
}

void UEditorValidatorSubsystem::ForEachEnabledValidator(TFunctionRef<bool(UEditorValidatorBase* Validator)> Callback) const
{
	UpdateValidators();

	for (const TPair<FTopLevelAssetPath, TObjectPtr<UEditorValidatorBase>>& ValidatorPair : Validators)
	{
		if (UEditorValidatorBase* Validator = ValidatorPair.Value;
			Validator && Validator->IsEnabled())
		{
			if (!Callback(Validator))
			{
				break;
			}
		}
	}
}

EDataValidationResult UEditorValidatorSubsystem::IsObjectValid(
	UObject* InObject,
	TArray<FText>& ValidationErrors,
	TArray<FText>& ValidationWarnings,
	const EDataValidationUsecase InValidationUsecase) const
{
	FDataValidationContext Context(false, InValidationUsecase, {}); // No associated objects in this context
	EDataValidationResult Result = IsObjectValidWithContext(InObject, Context);
	Context.SplitIssues(ValidationWarnings, ValidationErrors);
	return Result;
}

EDataValidationResult UEditorValidatorSubsystem::IsAssetValid(
	const FAssetData& AssetData,
	TArray<FText>& ValidationErrors,
	TArray<FText>& ValidationWarnings,
	const EDataValidationUsecase InValidationUsecase) const
{
	if (AssetData.IsValid())
	{
		UObject* Obj = AssetData.GetAsset({ ULevel::LoadAllExternalObjectsTag });
		if (Obj)
		{
			FDataValidationContext Context(false, InValidationUsecase, {}); // No associated objects in this context
			EDataValidationResult Result = ValidateObjectInternal(AssetData, Obj, Context);
			Context.SplitIssues(ValidationWarnings, ValidationErrors);
			return Result;
		}
		return EDataValidationResult::NotValidated;
	}

	return EDataValidationResult::Invalid;
}

EDataValidationResult UEditorValidatorSubsystem::IsObjectValidWithContext(
	UObject* InObject,
	FDataValidationContext& InContext) const
{
	EDataValidationResult Result = EDataValidationResult::NotValidated;
	
	if (ensure(InObject))
	{
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

		FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(InObject), true);
		if (!AssetData.IsValid())
		{
			// Construct dynamically with potentially fewer tags 
			AssetData = FAssetData(InObject);
		}
		
		return ValidateObjectInternal(AssetData, InObject, InContext);
	}

	return Result;
}

EDataValidationResult UEditorValidatorSubsystem::IsAssetValidWithContext(
	const FAssetData& AssetData,
	FDataValidationContext& InContext) const
{
	if (AssetData.IsValid())
	{
		UObject* Obj = AssetData.GetAsset({ ULevel::LoadAllExternalObjectsTag });
		if (Obj)
		{
			return ValidateObjectInternal(AssetData, Obj, InContext);
		}
		return EDataValidationResult::NotValidated;
	}

	return EDataValidationResult::Invalid;
}

EDataValidationResult UEditorValidatorSubsystem::ValidateObjectInternal(
	const FAssetData& InAssetData,
	UObject* InObject,
	FDataValidationContext& InContext,
	TMap<FTopLevelAssetPath, FValidatorStatistics>* ValidatorStatistics) const
{
	EDataValidationResult Result = EDataValidationResult::NotValidated;
	UE_LOG_CONTEXT("AssetBeingValidated", InAssetData.GetSoftObjectPath());
	
	if (ensure(InObject) && ensure(InAssetData.IsValid()))
	{
		// First check the class level validation
		Result = const_cast<const UObject*>(InObject)->IsDataValid(InContext);

		// If the asset is still valid or there wasn't a class-level validation, keep validating with custom validators
		if (Result == EDataValidationResult::Invalid)
		{
			return Result;
		}
		
		ForEachEnabledValidator([InObject, &InAssetData, &InContext, &Result, ValidatorStatistics](UEditorValidatorBase* Validator)
		{
			UE_LOGF(LogContentValidation, Verbose, "Validating '%ls' with '%ls'...", *InObject->GetPathName(), *Validator->GetClass()->GetName());
			GInitRunaway(); // Reset runaway counter, as ValidateLoadedAsset may be implemented in a BP and could overflow the runaway count due to being called in a loop

			EDataValidationResult NewResult = EDataValidationResult::NotValidated;
			if (ValidatorStatistics)
			{
				double ValidationDuration = 0.0;
				{
					FScopedDurationTimer ValidationTimer(ValidationDuration);
					NewResult = Validator->ValidateLoadedAsset(InAssetData, InObject, InContext);
				}

				FValidatorStatistics& ThisValidatorStats = ValidatorStatistics->FindOrAdd(Validator->GetClass()->GetClassPathName());
				if (NewResult != EDataValidationResult::NotValidated)
				{
					ThisValidatorStats.AssetsValidated++;
				}
				ThisValidatorStats.TotalDurationSeconds += ValidationDuration;
			}
			else
			{
				NewResult = Validator->ValidateLoadedAsset(InAssetData, InObject, InContext);
			}
			Result = CombineDataValidationResults(Result, NewResult);

			return true;
		});
	}

	return Result;
}

int32 UEditorValidatorSubsystem::ValidateAssetsWithSettings(
	const TArray<FAssetData>& AssetDataList,
	const FValidateAssetsSettings& InSettings,
	FValidateAssetsResults& OutResults) const
{
	const double StartTime = FPlatformTime::Seconds();
	EDataValidationResult Result = EDataValidationResult::NotValidated;
	UE::Telemetry::DataValidation::FValidateAssets Telemetry(InSettings);

	ON_SCOPE_EXIT
	{
		Telemetry.SetResults(Result, OutResults);
		Telemetry.Duration = FPlatformTime::Seconds() - StartTime;
		FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
	};

	FMessageLog DataValidationLog(InSettings.MessageLogName);
	DataValidationLog.SetCurrentPage(InSettings.MessageLogPageTitle);

	int32 NumMessagesPassingFilterPre = 0;
	if (const EMessageSeverity::Type* Severity = InSettings.ShowMessageLogSeverity.GetPtrOrNull())
	{
		NumMessagesPassingFilterPre = DataValidationLog.NumMessages(*Severity);
	}

	Result = ValidateAssetsInternal(DataValidationLog, TSet<FAssetData>{AssetDataList}, InSettings, OutResults);
	
	if (const EMessageSeverity::Type* Severity = InSettings.ShowMessageLogSeverity.GetPtrOrNull())
	{
		int32 NumMessagesPassingFilterPost = DataValidationLog.NumMessages(*Severity);
		if (NumMessagesPassingFilterPost > NumMessagesPassingFilterPre)
		{
			DataValidationLog.Open(*Severity, false);
		}
	}

	return OutResults.NumWarnings + OutResults.NumInvalid;
}

EDataValidationResult UEditorValidatorSubsystem::ValidateAssetsInternal(
	FMessageLog& 					DataValidationLog,
	TSet<FAssetData>				AssetDataList,
	const FValidateAssetsSettings& 	InSettings,
	FValidateAssetsResults& 		OutResults) const
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	// The number of assets to validate may decrease from merging when dealing with external objects but it shouldn't increase
	FScopedSlowTask SlowTask(AssetDataList.Num(), LOCTEXT("DataValidation.ValidateAssetsTask", "Validating Assets"));
	if (!InSettings.bSilent)
	{
		SlowTask.MakeDialog();
	}
	
	UE_LOGF(LogContentValidation, Log, "Enabled validators:");
	ForEachEnabledValidator([](UEditorValidatorBase* Validator)
	{
		UE_LOGF(LogContentValidation, Log, "\t%ls", *Validator->GetClass()->GetClassPathName().ToString());
		return true;
	});
	
	// Broadcast the Editor event before we start validating. This lets other systems (such as Sequencer) restore the state
	// of the level to what is actually saved on disk before performing validation.
	if (FEditorDelegates::OnPreAssetValidation.IsBound())
	{
		FEditorDelegates::OnPreAssetValidation.Broadcast();
	}
	
	// Filter external objects out from the asset data list to be validated indirectly via their outers 
	TMap<FAssetData, TArray<FAssetData>> AssetsToExternalObjects;
	for (auto It = AssetDataList.CreateIterator(); It; ++It)
	{
		if (It->GetOptionalOuterPathName().IsNone())
		{
			// Standalone asset, leave it in the list
			continue;
		}

		FAssetData OuterAsset = AssetRegistry.GetAssetByObjectPath(It->ToSoftObjectPath().GetWithoutSubPath(), true);
		if (!OuterAsset.IsValid())
		{
			// We can't validate this asset if we can't find the package to load it into
			It.RemoveCurrent();
			continue;
		}

		// Special case for level instances in world partition - if the outer asset we'd like to validate is loaded & streamed in to 
		// another world, validate that world instead. 
		if (UWorld* AssetWorld = Cast<UWorld>(OuterAsset.FastGetAsset(false)))
		{
			if (   AssetWorld->PersistentLevel 
				&& AssetWorld->PersistentLevel->OwningWorld != nullptr 
				&& AssetWorld->PersistentLevel->OwningWorld != AssetWorld)
			{
				OuterAsset = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AssetWorld->PersistentLevel->OwningWorld), true);
			}
		}
		AssetsToExternalObjects.FindOrAdd(OuterAsset).Add(*It);
		It.RemoveCurrent();
	}
	
	// Add any packages which contain those external objects to be validated
	{
		FDataValidationContext ValidationContext(false, InSettings.ValidationUsecase, {}, InSettings.MessageLogName);
		OutResults.NumExternalObjects = 0;
		for (auto It = AssetsToExternalObjects.CreateIterator(); It; ++It)
		{
			const TPair<FAssetData, TArray<FAssetData>>& Pair = *It;
			if (ShouldValidateAsset(Pair.Key, InSettings, ValidationContext))
			{
				AssetDataList.Add(Pair.Key);
				OutResults.NumExternalObjects += Pair.Value.Num();
			}
			else
			{
				UE_LOGF(LogContentValidation, Display, "Package %ls (owner of some external objects) being skipped for validation.", 
					*WriteToString<256>(Pair.Key.PackageName));
				It.RemoveCurrent();
			}
		}
		UE::DataValidation::AddAssetValidationMessages(DataValidationLog, ValidationContext);
		DataValidationLog.Flush();
	}

	// Dont let other async compilation warnings be attributed incorrectly to the package that is loading.
	WaitForAssetCompilationIfNecessary(InSettings.ValidationUsecase, !InSettings.bSilent);

	OutResults.NumRequested = AssetDataList.Num();
	
	UE_LOGF(LogContentValidation, Display, "Starting to validate %d assets (%d associated objects such as actors)", 
		OutResults.NumRequested,
		OutResults.NumExternalObjects);
	
	UE_LOGFMT_LOC(LogContentValidation, Log, "DataValidation.PreValidateStats.Preface", "Additional assets added for validation by {ValidatorCount} validators:",
		("ValidatorCount", OutResults.ValidatorStatistics.Num()));
	for (const TPair<FTopLevelAssetPath, FValidatorStatistics>& Pair : OutResults.ValidatorStatistics)
	{
		UE_LOGFMT(LogContentValidation, Log, "  {Validator} : {AssetsAddedForValidation}",
			("Validator", Pair.Key), ("AssetsAddedForValidation", Pair.Value.AssetsAddedForValidation));
	}
	
	EDataValidationResult Result = EDataValidationResult::NotValidated;

	const bool bLoadExternalObjects = InSettings.bLoadExternalObjectsForValidation && InSettings.bLoadAssetsForValidation;

	auto ShouldSkipAsset = [&InSettings, this](const FAssetData& Data) -> bool
	{
		if (Data.HasAnyPackageFlags(PKG_Cooked))
		{
			return true;
		}

		if (InSettings.bSkipExcludedDirectories && IsPathExcludedFromValidation(Data.PackageName.ToString()))
		{
			return true;
		}

		return false;
	};

	auto OnAssetsLoaded = [&InSettings, this]()
	{
		WaitForAssetCompilationIfNecessary(InSettings.ValidationUsecase, !InSettings.bSilent);
	};

	UE::DataValidation::FValidationAssetRange ValidationRange(AssetDataList, InSettings, ShouldSkipAsset, OnAssetsLoaded);

	OutResults.NumSkipped += ValidationRange.GetNumSkipped();

	// Now add to map or update as needed
	for (UE::DataValidation::FAssetToValidate Asset : ValidationRange)
	{
		const FAssetData& Data = *Asset.Data;

		if (!InSettings.bSilent)
		{
			SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("DataValidation.ValidatingFilename", "Validating {0}"), FText::FromString(IAssetTools::Get().GetUserFacingFullName(Data))));
		}

		DataValidationLog.Info()
			->AddToken(FAssetDataToken::Create(Data))
			->AddToken(FTextToken::Create(LOCTEXT("Data.ValidatingAsset", "Validating asset")));

		UObject* LoadedAsset = Data.FastGetAsset(false);

		TConstArrayView<FAssetData> ValidationExternalObjects;
		if (const TArray<FAssetData>* ValidationExternalObjectsPtr = AssetsToExternalObjects.Find(Data))
		{
			ValidationExternalObjects = *ValidationExternalObjectsPtr;
		}

		// This asset is about to be validated, so remove it from the latent save validation list
		if (SavedPackagesToValidate.Num() > 0)
		{
			SavedPackagesToValidate.Remove(Data.PackageName);
			for (const FAssetData& ExternalObject : ValidationExternalObjects)
			{
				SavedPackagesToValidate.Remove(ExternalObject.PackageName);
			}
		}

		bool bWasLoadedForValidation = !Asset.bWasAlreadyLoaded;
		FDataValidationContext ValidationContext(bWasLoadedForValidation, InSettings.ValidationUsecase, ValidationExternalObjects, InSettings.MessageLogName);
		EDataValidationResult AssetResult = EDataValidationResult::NotValidated;
		if (!LoadedAsset)
		{
			UE::DataValidation::LogLoadingAsset(Data, InSettings.bUnloadAssetsLoadedForValidation);
			UE::DataValidation::FScopedLogMessageGatherer LogGatherer(InSettings.bCaptureAssetLoadLogs);
			UE_LOG_CONTEXT("AssetBeingValidated", Data.GetSoftObjectPath());
			
			// Don't pass the flag to load all external objects. Later we'll load external objects which are also part of the to-validate
			// set if the settings allow for it.
			LoadedAsset = Data.GetAsset();

			bWasLoadedForValidation = true;

			WaitForAssetCompilationIfNecessary(InSettings.ValidationUsecase);

			// Associate any load errors with this asset in the message log
			TArray<FString> Warnings;
			TArray<FString> Errors;
			LogGatherer.Stop(Warnings, Errors);
			if (Warnings.Num())
			{
				TStringBuilder<2048> Buffer;
				Buffer.Join(Warnings, LINE_TERMINATOR);
				ValidationContext.AddMessage(EMessageSeverity::Warning)
					->AddToken(FAssetDataToken::Create(Data))
					->AddText(LOCTEXT("DataValidation.LoadWarnings", "Warnings loading asset {0}"), FText::FromStringView(Buffer.ToView()));
			}
			if(Errors.Num())
			{
				TStringBuilder<2048> Buffer;
				Buffer.Join(Errors, LINE_TERMINATOR);
				ValidationContext.AddMessage(EMessageSeverity::Error)
					->AddToken(FAssetDataToken::Create(Data))
					->AddText(LOCTEXT("DataValidation.LoadErrors", "Errors loading asset {0}"), FText::FromStringView(Buffer.ToView()));
				AssetResult = EDataValidationResult::Invalid;
			}
		}

		if (bWasLoadedForValidation)
		{
			ValidationContext.MarkAssetLoadedForValidation();

			if (InSettings.bUnloadAssetsLoadedForValidation)
			{
				ValidationRange.TrackPackageForUnload(LoadedAsset ? LoadedAsset->GetPackage() : nullptr);
			}
		}

		if (LoadedAsset)
		{
			UE::DataValidation::FScopedLoadExternalObjects ExternalObjectsLoader(LoadedAsset, ValidationContext, bLoadExternalObjects);

			UE::DataValidation::FScopedLogMessageGatherer LogGatherer(InSettings.bCaptureLogsDuringValidation);
			AssetResult = ValidateObjectInternal(Data, LoadedAsset, ValidationContext, &OutResults.ValidatorStatistics);
			
			// Associate any UE_LOG errors with this asset in the message log
			TArray<FString> Warnings;
			TArray<FString> Errors;
			LogGatherer.Stop(Warnings, Errors);
			if (Warnings.Num())
			{
				TStringBuilder<2048> Buffer;
				Buffer.Join(Warnings, LINE_TERMINATOR);
				ValidationContext.AddMessage(InSettings.bCaptureWarningsDuringValidationAsErrors ? EMessageSeverity::Error : EMessageSeverity::Warning)
					->AddToken(FAssetDataToken::Create(Data))
					->AddText(LOCTEXT("DataValidation.DuringValidationWarnings", "Warnings logged while validating asset {0}"), FText::FromStringView(Buffer.ToView()));
				if (InSettings.bCaptureWarningsDuringValidationAsErrors)
				{
					AssetResult = EDataValidationResult::Invalid;
				}
			}
			if(Errors.Num())
			{
				TStringBuilder<2048> Buffer;
				Buffer.Join(Errors, LINE_TERMINATOR);
				ValidationContext.AddMessage(EMessageSeverity::Error)
					->AddToken(FAssetDataToken::Create(Data))
					->AddText(LOCTEXT("DataValidation.DuringValidationErrors", "Errors logged while validating asset {0}"), FText::FromStringView(Buffer.ToView()));
				AssetResult = EDataValidationResult::Invalid;
			}
		}
		else if (InSettings.bLoadAssetsForValidation)
		{
			ValidationContext.AddMessage(EMessageSeverity::Error)
				->AddToken(FAssetDataToken::Create(Data))
				->AddToken(FTextToken::Create(LOCTEXT("DataValidation.LoadFailed", "Failed to load object")));
			AssetResult = EDataValidationResult::Invalid;
		}
		else 
		{
			ValidationContext.AddMessage(EMessageSeverity::Error)
				->AddToken(FAssetDataToken::Create(Data))
				->AddToken(FTextToken::Create(LOCTEXT("DataValidation.CannotValidateNotLoaded", "Cannot validate unloaded asset")));
			AssetResult = EDataValidationResult::Invalid;
		}

		++OutResults.NumChecked;
		
		// Don't add more messages to ValidationContext after this point because we will no longer add them to the message log 
		UE::DataValidation::AddAssetValidationMessages(Data, DataValidationLog, ValidationContext);

		bool bAnyWarnings = Algo::AnyOf(ValidationContext.GetIssues(), [](const FDataValidationContext::FIssue& Issue){ return Issue.Severity == EMessageSeverity::Warning; });
		if (bAnyWarnings)
		{
			++OutResults.NumWarnings;
		}

		if (InSettings.bShowIfNoFailures)
		{
			switch (AssetResult)
			{
				case EDataValidationResult::Valid:
					if (bAnyWarnings)
					{
						DataValidationLog.Info()->AddToken(FAssetDataToken::Create(Data))
							->AddToken(FTextToken::Create(LOCTEXT("DataValidation.ContainsWarningsResult", "contains valid data, but has warnings.")));
					}
					else 
					{
						DataValidationLog.Info()->AddToken(FAssetDataToken::Create(Data))
							->AddToken(FTextToken::Create(LOCTEXT("DataValidation.ValidResult", "contains valid data.")));
					}
					break;
				case EDataValidationResult::Invalid:
					DataValidationLog.Info()->AddToken(FAssetDataToken::Create(Data))
						->AddToken(FTextToken::Create(LOCTEXT("DataValidation.InvalidResult", "contains invalid data.")));
					break;
				case EDataValidationResult::NotValidated:
					DataValidationLog.Info()->AddToken(FAssetDataToken::Create(Data))
						->AddToken(FTextToken::Create(LOCTEXT("DataValidation.NotValidatedDataResult", "has no data validation.")));
					break;
			}
		}	

		switch (AssetResult)
		{
			case EDataValidationResult::Valid:
				++OutResults.NumValid;
				break;
			case EDataValidationResult::Invalid:
				++OutResults.NumInvalid;
				break;
			case EDataValidationResult::NotValidated:
				++OutResults.NumUnableToValidate;
				break;
		}

		if (InSettings.bCollectPerAssetDetails)
		{
			FValidateAssetsDetails& Details = OutResults.AssetsDetails.Emplace(Data.GetObjectPathString());
			Details.PackageName = Data.PackageName;
			Details.AssetName = Data.AssetName;
			Details.Result = AssetResult;
			ValidationContext.SplitIssues(Details.ValidationWarnings, Details.ValidationErrors, &Details.ValidationMessages);

			Details.ExternalObjects.Reserve(ValidationExternalObjects.Num());
			for (const FAssetData& ExtData : ValidationExternalObjects)
			{
				FValidateAssetsExternalObject& ExtDetails = Details.ExternalObjects.Emplace_GetRef();
				ExtDetails.PackageName = ExtData.PackageName;
				ExtDetails.AssetName = ExtData.AssetName;
			}
		}
		
		DataValidationLog.Flush();
		
		Result = CombineDataValidationResults(Result, AssetResult);
	}

	if (OutResults.NumChecked >= InSettings.MaxAssetsToValidate)
	{
		OutResults.bAssetLimitReached = true;
		DataValidationLog.Info(FText::Format(LOCTEXT("DataValidation.MaxAssetCountReached", "MaxAssetsToValidate count {0} reached."), InSettings.MaxAssetsToValidate));
	}

	int32 NumProcessed = ValidationRange.GetNumSkipped() + OutResults.NumChecked;
	if (OutResults.bAssetLimitReached)
	{
		ensureMsgf(NumProcessed <= AssetDataList.Num(),
			TEXT("Processed more assets than expected: %d skipped + %d checked > %d total"),
			ValidationRange.GetNumSkipped(), OutResults.NumChecked, AssetDataList.Num());
	}
	else
	{
		ensureMsgf(NumProcessed == AssetDataList.Num(),
			TEXT("Not all assets were processed: %d skipped + %d checked != %d total"),
			ValidationRange.GetNumSkipped(), OutResults.NumChecked, AssetDataList.Num());
	}

	ForEachEnabledValidator([&](UEditorValidatorBase* Validator)
	{
		Validator->PostAssetValidation(OutResults.ValidatorMessages);
		return true;
	});

	for (TSharedRef<FTokenizedMessage> const& Message : OutResults.ValidatorMessages)
	{
		DataValidationLog.AddMessage(Message);
	}
	
	UE_LOGFMT_LOC(LogContentValidation, Log, "DataValidation.PostValidateStats.Preface", "Validated asset counts for {ValidatorCount} validators:",
		("ValidatorCount", OutResults.ValidatorStatistics.Num()));
	for (const TPair<FTopLevelAssetPath, FValidatorStatistics>& Pair : OutResults.ValidatorStatistics)
	{
		UE_LOGFMT(LogContentValidation, Log, "  {Validator} : {AssetsValidated}",
			("Validator", Pair.Key), ("AssetsValidated", Pair.Value.AssetsValidated));
	}
	
	// Broadcast now that we're complete so other systems can go back to their previous state.
	if (FEditorDelegates::OnPostAssetValidation.IsBound())
	{
		FEditorDelegates::OnPostAssetValidation.Broadcast();
	}
	
	return Result;
}

void UEditorValidatorSubsystem::LogAssetValidationSummary(FMessageLog& DataValidationLog, const FValidateAssetsSettings& InSettings, EDataValidationResult Result, const FValidateAssetsResults& Results) const
{
	const bool bFailed = (Results.NumInvalid > 0) || Result != EDataValidationResult::Valid;
	const bool bAtLeastOneWarning = (Results.NumWarnings > 0);

	if (bFailed || bAtLeastOneWarning || InSettings.bShowIfNoFailures)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Result"), bFailed ? LOCTEXT("Failed", "FAILED") : LOCTEXT("Succeeded", "SUCCEEDED"));
		Arguments.Add(TEXT("NumChecked"), Results.NumChecked);
		Arguments.Add(TEXT("NumValid"), Results.NumValid);
		Arguments.Add(TEXT("NumInvalid"), Results.NumInvalid);
		Arguments.Add(TEXT("NumSkipped"), Results.NumSkipped);
		Arguments.Add(TEXT("NumUnableToValidate"), Results.NumUnableToValidate);

		DataValidationLog.Info()->AddToken(FTextToken::Create(FText::Format(LOCTEXT("DataValidation.SuccessOrFailure", "Data validation {Result}."), Arguments)));
		DataValidationLog.Info()->AddToken(FTextToken::Create(FText::Format(LOCTEXT("DataValidation.ResultsSummary", "Files Checked: {NumChecked}, Passed: {NumValid}, Failed: {NumInvalid}, Skipped: {NumSkipped}, Unable to validate: {NumUnableToValidate}"), Arguments)));

		DataValidationLog.Open(EMessageSeverity::Info, true);
	}
}

void UEditorValidatorSubsystem::ValidateOnSave(TArray<FAssetData> AssetDataList, bool bProceduralSave) const
{
	if (!ShouldValidateOnSave(bProceduralSave))
	{
		return;
	}
	
	FValidateAssetsSettings Settings;
	{
		FDataValidationContext Context {false, EDataValidationUsecase::Save, {}};
		AssetDataList.SetNum(Algo::RemoveIf(AssetDataList, [this, &Settings, &Context](const FAssetData& Asset) { 
			return !ShouldValidateAsset(Asset, Settings, Context); 
		}));
	}

	if (AssetDataList.IsEmpty())
	{
		return;
	}

	FText SavedAsset = AssetDataList.Num() == 1 ? FText::FromName(AssetDataList[0].AssetName) : LOCTEXT("MultipleAssets", "multiple assets");
	FValidateAssetsResults Results;

	Settings.bSkipExcludedDirectories = true;
	Settings.bShowIfNoFailures = false;
	Settings.ValidationUsecase = EDataValidationUsecase::Save;
	Settings.bLoadAssetsForValidation = false;
	Settings.MessageLogPageTitle = UE::DataValidation::MakeTimestampedMessageLogPageTitle(FText::Format(LOCTEXT("MessageLogPageTitle.ValidateSavedAssets", "Asset Save: {0}"), SavedAsset));

	const int32 NumAssetsWithIssues = ValidateAssetsWithSettings(AssetDataList, Settings, Results);

	if (NumAssetsWithIssues > 0 && !Settings.ShowMessageLogSeverity.IsSet()) // Only show the toast if ValidateAssetsWithSettings didn't already open the message log
	{
		FMessageLog DataValidationLog(Settings.MessageLogName);

		const FText ErrorMessageNotification = FText::Format(
			LOCTEXT("ValidationFailureNotification", "Validation failed when saving {0}, check Data Validation log"), SavedAsset);
		DataValidationLog.Notify(ErrorMessageNotification, EMessageSeverity::Warning, /*bForce=*/ true);
	}
}

void UEditorValidatorSubsystem::ValidateSavedPackage(FName PackageName, bool bProceduralSave)
{
	if (!ShouldValidateOnSave(bProceduralSave))
	{
		return;
	}

	if (SavedPackagesToValidate.Num() == 0)
	{
		GEditor->GetTimerManager()->SetTimerForNextTick(this, &UEditorValidatorSubsystem::ValidateAllSavedPackages);
	}

	SavedPackagesToValidate.Add(PackageName);
}

bool UEditorValidatorSubsystem::IsPathExcludedFromValidation(FStringView Path) const
{
	for (const FDirectoryPath& ExcludedPath : ExcludedDirectories)
	{
		if (Path.Contains(ExcludedPath.Path))
		{
			return true;
		}
	}

	return false;
}

void UEditorValidatorSubsystem::ValidateAllSavedPackages()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEditorValidatorSubsystem::ValidateAllSavedPackages);

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	// Prior to validation, make sure Asset Registry is updated. This is done by ticking the DirectoryWatcher module, which 
	// is responsible of scanning modified asset files.
	if( !FApp::IsProjectNameEmpty() )
	{
		static FName DirectoryWatcherName("DirectoryWatcher");
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(DirectoryWatcherName);
		DirectoryWatcherModule.Get()->Tick(1.f);
	}
	// We need to query the in-memory data as the disk cache may not be accurate
	FARFilter Filter;
	Filter.PackageNames = SavedPackagesToValidate.Array();
	Filter.bIncludeOnlyOnDiskAssets = false;

	TArray<FAssetData> Assets;
	AssetRegistry.GetAssets(Filter, Assets);

	SavedPackagesToValidate.Empty();

	bool bProceduralSave = false; // The optional suppression for ProceduralSaves was checked before adding to SavedPackagesToValidate
	ValidateOnSave(MoveTemp(Assets), bProceduralSave);
}

void UEditorValidatorSubsystem::ValidateChangelistPreSubmit(
	FSourceControlChangelistPtr InChangelist,
	FSourceControlValidationDelegateResult& OutResult) const
{
	check(InChangelist.IsValid());

	// Create temporary changelist object to do most of the heavy lifting
	UDataValidationChangelist* Changelist = NewObject<UDataValidationChangelist>();
	Changelist->Initialize(InChangelist);

	FValidateAssetsSettings Settings;
	Settings.ValidationUsecase = EDataValidationUsecase::PreSubmit;
	Settings.bLoadAssetsForValidation = GetDefault<UDataValidationSettings>()->bLoadAssetsWhenValidatingChangelists;
	Settings.MessageLogPageTitle = UE::DataValidation::MakeTimestampedMessageLogPageTitle(FText::Format(LOCTEXT("MessageLogPageTitle.ValidateChangelist", "Changelist Validation: {0}"), FText::FromString(InChangelist->GetIdentifier())));
	Settings.ShowMessageLogSeverity = { EMessageSeverity::Warning };
	Settings.bCollectPerAssetDetails = true;
	Settings.bShowIfNoFailures = false;

	FValidateAssetsResults Results;
	OutResult.Result = ValidateChangelist(Changelist, Settings, Results);
	
	for (const TPair<FString, FValidateAssetsDetails>& Pair : Results.AssetsDetails)
	{
		const FValidateAssetsDetails& Details = Pair.Value;

		OutResult.Warnings.Append(Details.ValidationWarnings);
		OutResult.Errors.Append(Details.ValidationErrors);
		
		for (TSharedRef<FTokenizedMessage> const& Message : Details.ValidationMessages)
		{
			OutResult.AllMessages.Add(Message->ToText());
		}
	}

	for (TSharedRef<FTokenizedMessage> const& Message : Results.ValidatorMessages)
	{
		OutResult.AllMessages.Add(Message->ToText());
	}
}

void UEditorValidatorSubsystem::ValidateFilesPreSubmit(
	TArray<FSourceControlStateRef>& InStates,
	FSourceControlValidationDelegateResult& OutResult) const
{
	// Create temporary changelist object to do most of the heavy lifting
	UDataValidationChangelist* Changelist = NewObject<UDataValidationChangelist>();
	Changelist->Initialize(InStates);

	FValidateAssetsSettings Settings;
	Settings.ValidationUsecase = EDataValidationUsecase::PreSubmit;
	Settings.bLoadAssetsForValidation = GetDefault<UDataValidationSettings>()->bLoadAssetsWhenValidatingChangelists;
	Settings.MessageLogPageTitle = UE::DataValidation::MakeTimestampedMessageLogPageTitle(LOCTEXT("MessageLogPageTitle.ValidateFiles", "Files Validation"));
	Settings.ShowMessageLogSeverity = { EMessageSeverity::Warning };
	Settings.bCollectPerAssetDetails = true;
	Settings.bShowIfNoFailures = false;

	FValidateAssetsResults Results;
	OutResult.Result = ValidateChangelist(Changelist, Settings, Results);

	for (const TPair<FString, FValidateAssetsDetails>& Pair : Results.AssetsDetails)
	{
		const FValidateAssetsDetails& Details = Pair.Value;

		OutResult.Warnings.Append(Details.ValidationWarnings);
		OutResult.Errors.Append(Details.ValidationErrors);

		for (TSharedRef<FTokenizedMessage> const& Message : Details.ValidationMessages)
		{
			if (Message->GetSeverity() == EMessageSeverity::Info)
			{
				OutResult.AllMessages.Add(Message->ToText());
			}
		}
	}

	for (TSharedRef<FTokenizedMessage> const& Message : Results.ValidatorMessages)
	{
		OutResult.AllMessages.Add(Message->ToText());
	}
}

EDataValidationResult UEditorValidatorSubsystem::ValidateChangelist(
	UDataValidationChangelist* InChangelist,
	const FValidateAssetsSettings& InSettings,
	FValidateAssetsResults& OutResults) const
{
	return ValidateChangelistsInternal(MakeArrayView(&InChangelist, 1), InSettings, OutResults);	
}

EDataValidationResult UEditorValidatorSubsystem::ValidateChangelists(
	const TArray<UDataValidationChangelist*> InChangelists,
	const FValidateAssetsSettings& InSettings,
	FValidateAssetsResults& OutResults) const
{
	return ValidateChangelistsInternal(MakeArrayView(InChangelists), InSettings, OutResults);	
}

EDataValidationResult UEditorValidatorSubsystem::ValidateChangelistsInternal(
	TConstArrayView<UDataValidationChangelist*>	Changelists,
	const FValidateAssetsSettings& 				Settings,
	FValidateAssetsResults& 					OutResults) const
{
	const double StartTime = FPlatformTime::Seconds();
	FScopedSlowTask SlowTask(Changelists.Num(), LOCTEXT("DataValidation.ValidatingChangelistTask", "Validating Changelists"));
	SlowTask.Visibility = ESlowTaskVisibility::Invisible;
	if (!Settings.bSilent)
	{
		SlowTask.MakeDialog();
	}

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	if (AssetRegistry.IsLoadingAssets())
	{
		UE_CLOGF(FApp::IsUnattended(), LogContentValidation, Fatal, "Unable to perform unattended content validation while asset registry scan is in progress. Callers just wait for asset registry scan to complete.");
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DataValidation.UnableToValidate_PendingAssetRegistry", "Unable to validate changelist while asset registry scan is in progress. Wait until asset discovery is complete."));
		return EDataValidationResult::NotValidated;
	}

	EDataValidationResult Result = EDataValidationResult::NotValidated;
	UE::Telemetry::DataValidation::FValidateAssets Telemetry(Settings);

	ON_SCOPE_EXIT
	{
		Telemetry.SetResults(Result, OutResults);
		Telemetry.Duration = FPlatformTime::Seconds() - StartTime;
		FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
	};

	FMessageLog DataValidationLog(Settings.MessageLogName);

	// Choose a specific message log page for this output, flushing in case other recursive calls also write to this log
	DataValidationLog.SetCurrentPage(Settings.MessageLogPageTitle);

	int32 NumMessagesPassingFilterPre = 0;
	if (const EMessageSeverity::Type* Severity = Settings.ShowMessageLogSeverity.GetPtrOrNull())
	{
		NumMessagesPassingFilterPre = DataValidationLog.NumMessages(*Severity);
	}

	for (UDataValidationChangelist* CL : Changelists)
	{
		CL->AddToRoot();
	}
	
	ON_SCOPE_EXIT 
	{
		for (UDataValidationChangelist* CL : Changelists)
		{
			CL->RemoveFromRoot();		
		}
	};

	for (UDataValidationChangelist* Changelist : Changelists)
	{
		FText ValidationMessage = FText::Format(LOCTEXT("DataValidation.ValidatingChangelistMessage", "Validating changelist {0}"), Changelist->Description);
		DataValidationLog.Info(ValidationMessage);
		if (!Settings.bSilent)
		{
			SlowTask.EnterProgressFrame(1.0f, ValidationMessage);
		}

		FValidateAssetsDetails& Details = OutResults.AssetsDetails.FindOrAdd(Changelist->GetPathName());
		{
			FDataValidationContext ValidationContext(false, Settings.ValidationUsecase, {}, Settings.MessageLogName); // No associated objects for changelist
			Details.Result = IsObjectValidWithContext(Changelist, ValidationContext);
			UE::DataValidation::AddAssetValidationMessages(DataValidationLog, ValidationContext);
			ValidationContext.SplitIssues(Details.ValidationWarnings, Details.ValidationErrors);
		}
		Result = CombineDataValidationResults(Result, Details.Result);
		DataValidationLog.Flush();
	}

	TSet<FAssetData> AssetsToValidate;	
	for (UDataValidationChangelist* Changelist : Changelists)
	{
		FDataValidationContext ValidationContext(false, Settings.ValidationUsecase, {}, Settings.MessageLogName); // No associated objects for changelist
		GatherAssetsToValidateFromChangelist(Changelist, Settings, AssetsToValidate, ValidationContext, OutResults);
		UE::DataValidation::AddAssetValidationMessages(DataValidationLog, ValidationContext);
		DataValidationLog.Flush();
	}

	// Filter out assets that we don't want to validate
	{
		FDataValidationContext ValidationContext(false, Settings.ValidationUsecase, {}, Settings.MessageLogName);
		for (auto It = AssetsToValidate.CreateIterator(); It; ++It)
		{
			if (!ShouldValidateAsset(*It, Settings, ValidationContext))
			{
				UE_LOGF(LogContentValidation, Display, "Excluding asset %ls from validation", *It->GetSoftObjectPath().ToString());
				It.RemoveCurrent();
			}
		}
		UE::DataValidation::AddAssetValidationMessages(DataValidationLog, ValidationContext);
		DataValidationLog.Flush();
	}

	// Validate assets from all changelists
	EDataValidationResult AssetResult = ValidateAssetsInternal(DataValidationLog, MoveTemp(AssetsToValidate), Settings, OutResults);
	Result = CombineDataValidationResults(Result, AssetResult);
	LogAssetValidationSummary(DataValidationLog, Settings, Result, OutResults);

	if (const EMessageSeverity::Type* Severity = Settings.ShowMessageLogSeverity.GetPtrOrNull())
	{
		int32 NumMessagesPassingFilterPost = DataValidationLog.NumMessages(*Severity);
		if (NumMessagesPassingFilterPost > NumMessagesPassingFilterPre)
		{
			DataValidationLog.Open(*Severity, false);
		}
	}

	return Result;
}

void UEditorValidatorSubsystem::GatherAssetsToValidateFromChangelist(
	UDataValidationChangelist* 		InChangelist,
	const FValidateAssetsSettings& 	Settings,
	TSet<FAssetData>& 				OutAssets,
	FDataValidationContext& 		InContext,
	FValidateAssetsResults& 		OutResults) const
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	
	for (const FName& PackageName : InChangelist->ModifiedPackageNames)
	{
		TArray<FAssetData> NewAssets;
		AssetRegistry.GetAssetsByPackageName(PackageName, NewAssets, true);	
		OutAssets.Append(NewAssets);
	}
	
	// Gather assets requested by plugin/project validators 
	ForEachEnabledValidator([this, InChangelist, &Settings, &InContext, &OutAssets, &OutResults](UEditorValidatorBase* Validator)
	{
		int32 OldCount = OutAssets.Num();
		TArray<FAssetData> NewAssets = Validator->GetAssetsToValidateFromChangelist(InChangelist, InContext);
		for (const FAssetData& Asset : NewAssets)
		{
			// It's not strictly necessary to filter assets here but it makes logging simpler
			if (ShouldValidateAsset(Asset, Settings, InContext))
			{
				bool bExisted = false;
				OutAssets.Add(Asset, &bExisted);
				UE_CLOGF(!bExisted, LogContentValidation, Display, "Asset validator %ls adding %ls to be validated from changelist %ls.", *Validator->GetPathName(), *Asset.GetSoftObjectPath().ToString(), *InChangelist->Description.ToString());
			}
		}
		const int32 NumAssetsAdded = OutAssets.Num() - OldCount;
		if (NumAssetsAdded)
		{
			FValidatorStatistics& Stats = OutResults.ValidatorStatistics.FindOrAdd(FTopLevelAssetPath(Validator->GetClass()));
			Stats.AssetsAddedForValidation += NumAssetsAdded;
		}
		return true;
	});
	
	if (Settings.bValidateReferencersOfDeletedAssets)
	{
		// Associate these with the validation subsystem base class - key type may need to change if we add more features here and want a breakdown
		FValidatorStatistics& Stats = OutResults.ValidatorStatistics.FindOrAdd(FTopLevelAssetPath(UEditorValidatorSubsystem::StaticClass()));
		int32 OldCount = OutAssets.Num();
		TSet<FAssetData> AdditionalAssetsToValidate;
		for (FName DeletedPackageName : InChangelist->DeletedPackageNames)
		{
			TArray<FName> PackageReferencers;
			AssetRegistry.GetReferencers(DeletedPackageName, PackageReferencers, UE::AssetRegistry::EDependencyCategory::Package);
			if (PackageReferencers.Num() > 0)
			{
				InContext.AddMessage(EMessageSeverity::Info, 
					FText::Format(LOCTEXT("DataValidation.AddDeletedPackageReferencers", "Adding referencers of deleted package {0} to be validated"), FText::FromName(DeletedPackageName)));
			}
			for (const FName& Referencer : PackageReferencers)
			{
				UE_LOGF(LogContentValidation, Display, "Adding %ls to to validated as it is a referencer of deleted asset %ls", *Referencer.ToString(), *DeletedPackageName.ToString());
				TArray<FAssetData> NewAssets;
				AssetRegistry.GetAssetsByPackageName(Referencer, NewAssets, true);
				OutAssets.Append(NewAssets);
			}
		}
		Stats.AssetsAddedForValidation += OutAssets.Num() - OldCount;
	}
}

void UEditorValidatorSubsystem::OnNativeModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason)
{
	switch (InModuleChangeReason)
	{
		case EModuleChangeReason::ModuleLoaded:
			NativeModulesPendingLoad.Add(InModuleName);
			NativeModulesPendingUnload.Remove(InModuleName);
			break;

		case EModuleChangeReason::ModuleUnloaded:
			NativeModulesPendingUnload.Add(InModuleName);
			NativeModulesPendingLoad.Remove(InModuleName);
			break;

		default:
			break;
	}
}

void UEditorValidatorSubsystem::OnAssetsAdded(TConstArrayView<FAssetData> InAssets)
{
	OnAssetsAddedOrRemoved(InAssets, [this](const FTopLevelAssetPath& BPGC) { 
		UE_LOGFMT(LogContentValidation, Log, "Adding validator from newly added asset {ValidatorClass}", BPGC);
		AddValidator_Internal(BPGC); 
	});
}

void UEditorValidatorSubsystem::OnAssetsRemoved(TConstArrayView<FAssetData> InAssets)
{
	OnAssetsAddedOrRemoved(InAssets, [this](const FTopLevelAssetPath& BPGC) {
		UE_LOGFMT(LogContentValidation, Log, "Removing validator from removed asset {ValidatorClass}", BPGC);
		RemoveValidator_Internal(BPGC); 
	});
}

void UEditorValidatorSubsystem::OnAssetsAddedOrRemoved(TConstArrayView<FAssetData> InAssets, TFunctionRef<void(const FTopLevelAssetPath& BPGC)> Callback)
{
	for (const FAssetData& Asset : InAssets)
	{
		if (const UClass* AssetClass = Asset.GetClass())
		{
			if (AssetClass->IsChildOf<UEditorUtilityBlueprint>())
			{
				// Uncooked BP
				if (const UClass* ParentClass = UBlueprint::GetBlueprintParentClassFromAssetTags(Asset);
					ParentClass && ParentClass->IsChildOf<UEditorValidatorBase>())
				{
					FTopLevelAssetPath BPGC = FEditorClassUtils::GetClassPathNameFromAssetTag(Asset);
					if (BPGC.IsNull())
					{
						BPGC = FTopLevelAssetPath(Asset.PackageName, FName(WriteToString<128>(Asset.AssetName, TEXT("_C"))));
					}
					Callback(BPGC);
				}
			}
			else if (AssetClass->IsChildOf<UBlueprintGeneratedClass>())
			{
				// Cooked BPGC
				if (const UClass* ParentClass = UBlueprint::GetBlueprintParentClassFromAssetTags(Asset);
					ParentClass && ParentClass->IsChildOf<UEditorValidatorBase>())
				{
					Callback(FTopLevelAssetPath(Asset.PackageName, Asset.AssetName));
				}
			}
		}
	}
}

void UEditorValidatorSubsystem::UpdateValidators() const
{
	const_cast<UEditorValidatorSubsystem*>(this)->UpdateValidators();
}

void UEditorValidatorSubsystem::UpdateValidators()
{
	UE_CLOGF(!bHasRegisteredNativeValidators, LogContentValidation, Warning, "UpdateValidators request made before RegisterNativeValidators. Native validators may be missing!");
	UE_CLOGF(!bHasRegisteredBlueprintValidators, LogContentValidation, Warning, "UpdateValidators request made before RegisterBlueprintValidators. Blueprint validators may be missing!");

	// Remove any existing validators for unloaded modules
	if (NativeModulesPendingUnload.Num() > 0)
	{
		TSet<FName> ModulePackageNames;
		ModulePackageNames.Reserve(NativeModulesPendingUnload.Num());
		Algo::Transform(NativeModulesPendingUnload, ModulePackageNames, [](const FName ModuleName) { return FPackageName::GetModuleScriptPackageName(ModuleName); });

		for (auto It = Validators.CreateIterator(); It; ++It)
		{
			if (ModulePackageNames.Contains(It->Key.GetPackageName()))
			{
				It.RemoveCurrent();
			}
		}

		NativeModulesPendingUnload.Reset();
	}

	// Add any new validators for loaded modules
	if (NativeModulesPendingLoad.Num() > 0)
	{
		TSet<FName> ModulePackageNames;
		ModulePackageNames.Reserve(NativeModulesPendingLoad.Num());
		Algo::Transform(NativeModulesPendingLoad, ModulePackageNames, [](const FName ModuleName) { return FPackageName::GetModuleScriptPackageName(ModuleName); });

		// GetDerivedClasses has an accelerator table, so it's faster to query all the native UEditorValidatorBase classes (of which there will be relatively few) and 
		// then filter them down by module, than it is to get all the classes of each pending module (of which there may be many) and then filter them by type
		TArray<UClass*> ValidatorClasses;
		GetDerivedClasses(UEditorValidatorBase::StaticClass(), ValidatorClasses);

		for (UClass* ValidatorClass : ValidatorClasses)
		{
			// GetDerivedClasses may include a mix of C++ and loaded BP classes
			// Skip any classes outside of the modules requested, as well as anything that has already been registered by this point
			if (!ValidatorClass->HasAllClassFlags(CLASS_Abstract) && !Validators.Contains(ValidatorClass->GetClassPathName())
				&& ModulePackageNames.Contains(ValidatorClass->GetPackage()->GetFName()))
			{
				UEditorValidatorBase* Validator = NewObject<UEditorValidatorBase>(GetTransientPackage(), ValidatorClass);
				UE_LOGFMT(LogContentValidation, Log, "Adding native validator {ValidatorClass} from newly loaded module", FTopLevelAssetPath(ValidatorClass));
				AddValidator_Internal(Validator);
			}
		}

		NativeModulesPendingLoad.Reset();
	}

	// Add any new validators from pending Blueprint classes
	if (ValidatorClassesPendingLoad.Num() > 0)
	{
		FScopedSlowTask SlowTask(ValidatorClassesPendingLoad.Num(), LOCTEXT("DataValidation.LoadingValidatorsTask", "Loading validators..."));
		SlowTask.MakeDialog();
		for (const FTopLevelAssetPath& ValidatorClassPendingLoad : ValidatorClassesPendingLoad)
		{
			SlowTask.EnterProgressFrame(1.0f);
			if (TObjectPtr<UEditorValidatorBase>* ValidatorInstance = Validators.Find(ValidatorClassPendingLoad);
				ensure(ValidatorInstance))
			{
				TSoftClassPtr<UEditorValidatorBase> ValidatorClassSoftPtr = TSoftClassPtr<UEditorValidatorBase>(FSoftObjectPath(ValidatorClassPendingLoad));
				UClass* ValidatorClass = ValidatorClassSoftPtr.Get();

				// If this class isn't currently loaded, load it
				if (!ValidatorClass)
				{
					FCookLoadScope EditorOnlyLoadScope(ECookLoadType::EditorOnly);
					FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::EditorOnlyCollect, ESoftObjectPathSerializeType::AlwaysSerialize);
					ValidatorClass = ValidatorClassSoftPtr.LoadSynchronous();
				}

				if (ValidatorClass && !ValidatorClass->HasAnyClassFlags(CLASS_Abstract))
				{
					UE_LOGFMT(LogContentValidation, Log, "Creating validator instance {ValidatorClass}", FTopLevelAssetPath(ValidatorClass));
					*ValidatorInstance = NewObject<UEditorValidatorBase>(GetTransientPackage(), ValidatorClass);
				}
			}
		}
		ValidatorClassesPendingLoad.Reset();
	}
}

TArray<FAssetData> UEditorValidatorSubsystem::GetAssetsResolvingRedirectors(FARFilter& InFilter)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	TArray<FAssetData> Found;
	AssetRegistry.GetAssets(InFilter, Found);
	
	TArray<FAssetData> Redirectors;	
	for (int32 i=Found.Num()-1; i >= 0; --i)
	{
		if (Found[i].IsRedirector())
		{
			Redirectors.Add(Found[i]);
			Found.RemoveAt(i);
		}
	}
	
	for (const FAssetData& RedirectorAsset : Redirectors)
	{
		FSoftObjectPath Path = AssetRegistry.GetRedirectedObjectPath(RedirectorAsset.GetSoftObjectPath());		
		if (!Path.IsNull())
		{
			FAssetData Destination = AssetRegistry.GetAssetByObjectPath(Path, true);
			if (Destination.IsValid())
			{
				Found.Add(Destination);
			}
		}
	}
	return Found;
}

void UEditorValidatorSubsystem::WaitForAssetCompilationIfNecessary(EDataValidationUsecase InUsecase, bool bShowProgress) const
{
	if (InUsecase == EDataValidationUsecase::Save)
	{
		return;
	}

	if (FAssetCompilingManager::Get().GetNumRemainingAssets())
	{
		if (bShowProgress)
		{
			FScopedSlowTask CompileAssetsSlowTask(0.f, LOCTEXT("DataValidation.CompilingAssetsBeforeCheckingContentTask", "Finishing asset compilations before checking content..."));
			CompileAssetsSlowTask.MakeDialog();
		}
		FAssetCompilingManager::Get().FinishAllCompilation();
	}
}

#undef LOCTEXT_NAMESPACE

