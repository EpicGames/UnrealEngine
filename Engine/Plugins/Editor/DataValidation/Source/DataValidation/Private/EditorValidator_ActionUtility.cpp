// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorValidator_ActionUtility.h"

#include "ActorActionUtility.h"
#include "AssetActionUtility.h"
#include "Engine/Blueprint.h"
#include "EditorValidatorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorValidator_ActionUtility)

#define LOCTEXT_NAMESPACE "ActionUtilityValidator"

namespace EditorValidatorActionUtilityHelpers
{
	UObject* GetObjectFromAssetPtr(UObject* InAsset)
	{
		UObject* Object = InAsset;
		if (const UBlueprint* BP = Cast<UBlueprint>(InAsset))
		{
			Object = BP->GeneratedClass.GetDefaultObject();
		}
		
		return Object;
	}
};

bool UEditorValidator_ActionUtility::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	check(InAsset);
	UObject* Object = EditorValidatorActionUtilityHelpers::GetObjectFromAssetPtr(InAsset);
	return Cast<UAssetActionUtility>(Object) || Cast<UActorActionUtility>(Object);
}

EDataValidationResult UEditorValidator_ActionUtility::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	check(InAsset);
	EDataValidationResult Result = EDataValidationResult::Valid;
	UObject* Object = EditorValidatorActionUtilityHelpers::GetObjectFromAssetPtr(InAsset);
	
	if (UActorActionUtility* ActorActionUtility = Cast<UActorActionUtility>(Object))
	{
		if (ActorActionUtility->GetSupportedClasses().IsEmpty())
		{
			AssetFails(Object, LOCTEXT("ActorActionUtility_NoSupportedClasses", 
				"Supported Classes array must not be empty. "
				"Add only the class(es) that your action supports. "
				"If your action is truly compatible with all types of Actors, add the 'Actor' class explicitly"));
			Result = EDataValidationResult::Invalid;
		}
	}
	else if (UAssetActionUtility* AssetActionUtility = Cast<UAssetActionUtility>(Object))
	{
		if (AssetActionUtility->GetSupportedClasses().IsEmpty())
		{
			AssetFails(Object, LOCTEXT("AssetActionUtility_NoSupportedClasses", 
				"Supported Classes array must not be empty. "
				"Add only the class(es) that your action supports. "
				"If your action is truly compatible with all types of Assets, add the 'Object' class explicitly"));
			Result = EDataValidationResult::Invalid;
		}
	}
	
	return Result;
}

#undef LOCTEXT_NAMESPACE
