// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorValidator_Localization.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Internationalization/Culture.h"
#include "Internationalization/PackageLocalizationUtil.h"
#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorValidator_Localization)

#define LOCTEXT_NAMESPACE "AssetValidation"

UEditorValidator_Localization::UEditorValidator_Localization()
	: Super()
{
	bIsEnabled = true;
}

bool UEditorValidator_Localization::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	return true;
}

EDataValidationResult UEditorValidator_Localization::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	static const FName NAME_AssetToools = "AssetTools";
	IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>(NAME_AssetToools).Get();

	static const FName NAME_AssetRegistry = "AssetRegistry";
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(NAME_AssetRegistry).Get();

	const FString AssetPathName = InAsset->GetPathName();
	
	const UPackage* AssetPackage = InAsset->GetOutermost();
	if (AssetPackage && FPackageName::IsLocalizedPackage(AssetPackage->GetPathName()))
	{
		// Get the source path for this asset
		FString SourceObjectPath;
		if (FPackageLocalizationUtil::ConvertLocalizedToSource(AssetPathName, SourceObjectPath))
		{
			const FAssetData SourceAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(SourceObjectPath));

			// Does this source asset exist?
			// It is valid to have orphan localized assets, as they may be a direct reference of another localized asset
			if (SourceAssetData.IsValid())
			{
				// Can this type of asset be localized?
				if (!AssetTools.CanLocalize(InAsset->GetClass()))
				{
					AssetFails(InAsset, FText::Format(LOCTEXT("LocalizationError_LocalizedAssetTypeCannotBeLocalized", "Localized asset is of type '{0}', which is not a type that can be localized!"),
						FText::FromString(InAsset->GetClass()->GetName())));
					return EDataValidationResult::Invalid;
				}

				// Is the source asset a redirector?
				if (SourceAssetData.IsRedirector())
				{
					// If this asset is a redirector, and the source asset is a redirector, then make sure this asset points to the localized version of what the source asset points to
					bool bIsRedirectorReferringToRedirectedLocation = false;
					if (UObjectRedirector* ThisRedirector = Cast<UObjectRedirector>(InAsset))
					{
						if (UObject* ThisDestinationObject = ThisRedirector->DestinationObject)
						{
							FString ThisDestinationSourceObjectPath;
							const UPackage* DestinationObjectPackage = ThisDestinationObject->GetOutermost();
							if (DestinationObjectPackage && FPackageName::IsLocalizedPackage(DestinationObjectPackage->GetPathName()) 
								&& FPackageLocalizationUtil::ConvertLocalizedToSource(ThisDestinationObject->GetPathName(), ThisDestinationSourceObjectPath))
							{
								FString SourceRedirectorDest;
								if (SourceAssetData.GetTagValue(TEXT("DestinationObject"), SourceRedirectorDest))
								{
									ConstructorHelpers::StripObjectClass(SourceRedirectorDest);

									bIsRedirectorReferringToRedirectedLocation = (ThisDestinationSourceObjectPath == SourceRedirectorDest);
								}
							}
						}
					}

					if (!bIsRedirectorReferringToRedirectedLocation)
					{
						AssetFails(InAsset, 
							LOCTEXT("LocalizationError_LocalizedAssetHasSourceRedirector", "Localized asset has a source asset that is a redirector. Did you forget to rename the localized assets too?"));
						return EDataValidationResult::Invalid;
					}
				}

				// Is the source asset the expected type?
				if (SourceAssetData.AssetClassPath != InAsset->GetClass()->GetClassPathName())
				{
					AssetFails(InAsset, FText::Format(LOCTEXT("LocalizationError_LocalizedTypeMismatchWithSourceAsset", 
							"Localized asset is of type '{0}', but its source asset is of type '{1}'. A localized asset must have the same type as its source asset!"),
							FText::FromString(InAsset->GetClass()->GetPathName()), FText::FromString(SourceAssetData.AssetClassPath.ToString())));
					return EDataValidationResult::Invalid;
				}
			}

			AssetPasses(InAsset);
			return EDataValidationResult::Valid;
		}
	}
	else
	{
		FString AssetLocalizationRoot;
		if (FPackageLocalizationUtil::GetLocalizedRoot(AssetPathName, FString(), AssetLocalizationRoot))
		{
			if (const TArray<FString>* CultureNames = FindOrCacheCulturesForLocalizedRoot(AssetLocalizationRoot))
			{
				for (const FString& CultureName : *CultureNames)
				{
					// Get the localized path for this asset and culture
					FString LocalizedObjectPath;
					if (FPackageLocalizationUtil::ConvertSourceToLocalized(AssetPathName, CultureName, LocalizedObjectPath))
					{
						const FAssetData LocalizedAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(LocalizedObjectPath));

						if (LocalizedAssetData.IsValid())
						{
							// Can this type of asset be localized?
							if (!AssetTools.CanLocalize(InAsset->GetClass()))
							{
								AssetFails(InAsset, FText::Format(LOCTEXT("LocalizationError_SourceAssetTypeCannotBeLocalized", 
									"Source asset has a localized asset for '{0}', but is of type '{1}' which is not a type that can be localized!"),
									FText::FromString(CultureName), FText::FromString(InAsset->GetClass()->GetName())));
								return EDataValidationResult::Invalid;
							}

							// Is the source asset a redirector?
							if (UObjectRedirector* ThisRedirector = Cast<UObjectRedirector>(InAsset))
							{
								// If this source is a redirector, and the localized asset is a redirector, then make sure this asset points to the source version of what the localized asset points to
								bool bIsRedirectorReferringToRedirectedLocation = false;
								if (LocalizedAssetData.IsRedirector())
								{
									if (UObject* ThisDestinationObject = ThisRedirector->DestinationObject)
									{
										FString ThisDestinationLocalizedObjectPath;
										if (FPackageLocalizationUtil::ConvertSourceToLocalized(ThisDestinationObject->GetPathName(), CultureName, ThisDestinationLocalizedObjectPath))
										{
											FString LocalizedRedirectorDest;
											if (LocalizedAssetData.GetTagValue(TEXT("DestinationObject"), LocalizedRedirectorDest))
											{
												ConstructorHelpers::StripObjectClass(LocalizedRedirectorDest);

												bIsRedirectorReferringToRedirectedLocation = (ThisDestinationLocalizedObjectPath == LocalizedRedirectorDest);
											}
										}
									}
								}

								if (!bIsRedirectorReferringToRedirectedLocation)
								{
									AssetFails(InAsset, FText::Format(LOCTEXT("LocalizationError_SourceIsRedirector", 
										"Source asset is a redirector but has a localized asset for '{0}'. Did you forget to rename the localized assets too?"), 
										FText::FromString(CultureName)));
									return EDataValidationResult::Invalid;
								}
							}

							// Is the localized asset the expected type?
							if (LocalizedAssetData.AssetClassPath != InAsset->GetClass()->GetClassPathName())
							{
								AssetFails(InAsset, FText::Format(LOCTEXT("LocalizationError_SourceTypeMismatchWithLocalizedAsset", 
									"Source asset is of type '{0}', but its localized asset for '{1}' is of type '{2}'. A localized asset must have the same type as its source asset!"), 
									FText::FromString(InAsset->GetClass()->GetPathName()), FText::FromString(CultureName), FText::FromString(LocalizedAssetData.AssetClassPath.ToString())));
								return EDataValidationResult::Invalid;
							}
						}
					}
				}

				AssetPasses(InAsset);
				return EDataValidationResult::Valid;
			}
		}
	}

	return EDataValidationResult::NotValidated;
}

const TArray<FString>* UEditorValidator_Localization::FindOrCacheCulturesForLocalizedRoot(const FString& InLocalizedRootPath)
{
	if (const TArray<FString>* CachedCultures = CachedCulturesForLocalizedRoots.Find(InLocalizedRootPath))
	{
		return CachedCultures;
	}

	FString LocalizedRootPathFilePath;
	if (FPackageName::TryConvertLongPackageNameToFilename(InLocalizedRootPath, LocalizedRootPathFilePath))
	{
		TArray<FString> CultureNames;
		IFileManager::Get().IterateDirectory(*LocalizedRootPathFilePath, [&CultureNames](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				// UE localization resource folders use "en-US" style while ICU uses "en_US"
				FString LocalizationFolder = FPaths::GetCleanFilename(FilenameOrDirectory);
				FString CanonicalName = FCulture::GetCanonicalName(LocalizationFolder);
				CultureNames.AddUnique(MoveTemp(CanonicalName));
			}
			return true;
		});
		CultureNames.Sort();

		return &CachedCulturesForLocalizedRoots.Add(InLocalizedRootPath, MoveTemp(CultureNames));
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

