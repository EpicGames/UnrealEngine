// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorValidator_Material.h"
#include "AssetCompilingManager.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DetailWidgetRow.h"
#include "EditorValidatorSubsystem.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/DataValidation.h"
#include "Widgets/Input/SComboBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorValidator_Material)

#define LOCTEXT_NAMESPACE "MaterialValidator"

static TAutoConsoleVariable<bool> bCVarEdAllowMaterialValidator(
	TEXT("Editor.EnableMaterialAssetValidator"),
	false,
	TEXT("Enables material asset validator in validation contexts where it's normally disabled (on save and from commandlet)."));

UEditorValidator_Material::UEditorValidator_Material()
	: Super()
{
	if (GetDefault<UDataValidationSettings>()->bEnableMaterialValidation)
	{
		for (const FMaterialEditorValidationPlatform& Config: GetDefault<UDataValidationSettings>()->MaterialValidationPlatforms)
		{
			FShaderValidationPlatform Platform = {};
			Platform.ShaderPlatformName = Config.ShaderPlatform.Name; 

			bool bValidShaderPlatform = false;
			if (Config.ShaderPlatform.Name == FMaterialEditorValidationShaderPlatform::MaxRHIShaderPlatformName)
			{
				Platform.ShaderPlatform = GMaxRHIShaderPlatform;
				bValidShaderPlatform = true;
			}
			else
			{
				for (int32 ShaderPlatformIndex = 0; ShaderPlatformIndex < SP_NumPlatforms; ++ShaderPlatformIndex)
				{
					const EShaderPlatform ShaderPlatform = static_cast<EShaderPlatform>(ShaderPlatformIndex);

					if (FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform)
						&& FDataDrivenShaderPlatformInfo::CanUseForMaterialValidation(ShaderPlatform)
						&& FDataDrivenShaderPlatformInfo::GetName(ShaderPlatform) == Config.ShaderPlatform.Name)
					{
						Platform.ShaderPlatform = ShaderPlatform;
						bValidShaderPlatform = true;
						break;
					}
				}
			}

			if (!bValidShaderPlatform)
			{
				UE_LOGF(LogContentValidation, Warning, "Material asset validation shader platform '%ls' is not available, skipping.", *Config.ShaderPlatform.Name.ToString());
				continue;
			}

			switch (Config.FeatureLevel)
			{
			case EMaterialEditorValidationFeatureLevel::CurrentMaxFeatureLevel: Platform.FeatureLevel = GMaxRHIFeatureLevel; break; 
			case EMaterialEditorValidationFeatureLevel::ES3_1: Platform.FeatureLevel = ERHIFeatureLevel::ES3_1; break;
			case EMaterialEditorValidationFeatureLevel::SM5: Platform.FeatureLevel = ERHIFeatureLevel::SM5; break;
			case EMaterialEditorValidationFeatureLevel::SM6: Platform.FeatureLevel = ERHIFeatureLevel::SM6; break;
			}

			switch (Config.MaterialQualityLevel)
			{
			case EMaterialEditorValidationQualityLevel::Low: Platform.MaterialQualityLevel = EMaterialQualityLevel::Low; break;
			case EMaterialEditorValidationQualityLevel::Medium: Platform.MaterialQualityLevel = EMaterialQualityLevel::Medium; break;
			case EMaterialEditorValidationQualityLevel::High: Platform.MaterialQualityLevel = EMaterialQualityLevel::High; break;
			case EMaterialEditorValidationQualityLevel::Epic: Platform.MaterialQualityLevel = EMaterialQualityLevel::Epic; break;
			}

			ValidationPlatforms.Add(Platform);
		}
	}
}

bool UEditorValidator_Material::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	if (!bCVarEdAllowMaterialValidator.GetValueOnAnyThread() &&
		(InContext.GetValidationUsecase() == EDataValidationUsecase::Save || InContext.GetValidationUsecase() == EDataValidationUsecase::Commandlet))
	{
		return false;
	}

	if (ValidationPlatforms.IsEmpty())
	{
		return false;
	}

	const UMaterial* OriginalMaterial = Cast<UMaterial>(InAsset);
	if (OriginalMaterial)
	{
		// We want to validate every UMaterial.
		return true;
	}

	const UMaterialInstance* OriginalMaterialInstance = Cast<UMaterialInstance>(InAsset);
	if (OriginalMaterialInstance)
	{
		FMaterialInheritanceChain Chain;
		OriginalMaterialInstance->GetMaterialInheritanceChain(Chain);

		for (const UMaterialInstance* MaterialInstance: Chain.MaterialInstances)
		{
			if (MaterialInstance->HasStaticParameters())
			{
				// We want to validate UMaterialInstance since it has a static parameter that could influence generated shader code.
				return true;
			}
		}
	}

	return false;
}

EDataValidationResult UEditorValidator_Material::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	UMaterialInstance* OriginalMaterialInstance = Cast<UMaterialInstance>(InAsset); // Can be null
	UMaterial* OriginalMaterial = OriginalMaterialInstance ? OriginalMaterialInstance->GetMaterial() : Cast<UMaterial>(InAsset);

	if (!ensureAlways(OriginalMaterial))
	{
		return EDataValidationResult::NotValidated;
	}

	const bool bAllowCompilingShaders = GetDefault<UDataValidationSettings>()->bMaterialValidationAllowCompilingShaders;
	const bool bShowWarningsWhenNotCompilingShaders = GetDefault<UDataValidationSettings>()->bMaterialValidationShowWarningsWhenNotCompilingShaders;

	bool bTranslationFailed = false;
	TArray<int32> RetryWithCompilation;
	TMap<FString, TArray<FName>> TranslationErrorMap;

	TArray<FMaterialResource*> TranslationResources;
	for (int32 ValidationPlatformIndex = 0; ValidationPlatformIndex < ValidationPlatforms.Num(); ++ValidationPlatformIndex)
	{
		const FShaderValidationPlatform& ValidationPlatform = ValidationPlatforms[ValidationPlatformIndex];

		FMaterialResource* MaterialResource = FindOrCreateMaterialResource(TranslationResources, OriginalMaterial, OriginalMaterialInstance, ValidationPlatform.ShaderPlatform, ValidationPlatform.MaterialQualityLevel);

		FMaterialAnalysisResult Result;
		// Translate in validation mode only when not allowing compiling shaders, otherwise the estimates can be higher and trigger unnecessary compilations.
		const bool bValidationMode = !bAllowCompilingShaders;
		OriginalMaterial->AnalyzeMaterialTranslationOutput(MaterialResource, ValidationPlatform.ShaderPlatform, bValidationMode, Result);

		if (!Result.bTranslationSuccess) // translation failed, report errors and early out
		{
			for (const FString& ErrorText: MaterialResource->GetCompileErrors())
			{
				TranslationErrorMap.FindOrAdd(ErrorText).AddUnique(ValidationPlatform.ShaderPlatformName);
			}

			if (MaterialResource->GetCompileErrors().IsEmpty())
			{
				AssetFails(InAsset, LOCTEXT("MaterialValidator_TranslationFailedButNoError", "Material translation failed with no error."));
			}

			bTranslationFailed = true;
		}
		else // translation success, check if we run out of samplers
		{
			const uint32 MaxSamplers = FDataDrivenShaderPlatformInfo::GetMaxSamplers(ValidationPlatform.ShaderPlatform);

			const bool bTooManyEstimatedSamplers = Result.EstimatedNumTextureSamplesVS > MaxSamplers || Result.EstimatedNumTextureSamplesPS > MaxSamplers;

			if (bAllowCompilingShaders && bTooManyEstimatedSamplers)
			{
				RetryWithCompilation.Add(ValidationPlatformIndex);
			}
			else if (bShowWarningsWhenNotCompilingShaders)
			{
				// We want to show these to users only as warnings. Because they are based on estimated numbers and can generate false positive errors.

				if (Result.EstimatedNumTextureSamplesVS > MaxSamplers)
				{
					AssetWarning(InAsset, FText::Format(
						LOCTEXT("MaterialValidator_EstimatedVSOverPlatformLimit", "Estimated amount of VS samplers ({0}) is larger than supported on shader platform ({1}), shader will likely not compile for a shader platform '{2}'."),
						Result.EstimatedNumTextureSamplesVS, MaxSamplers, FText::FromName(ValidationPlatform.ShaderPlatformName)
					));
				}

				if (Result.EstimatedNumTextureSamplesPS > MaxSamplers)
				{
					AssetWarning(InAsset, FText::Format(
						LOCTEXT("MaterialValidator_EstimatedPSOverPlatformLimit", "Estimated amount of PS samplers ({0}) is larger than supported on shader platform ({1}), shader will likely not compile for a shader platform '{2}'."),
						Result.EstimatedNumTextureSamplesPS, MaxSamplers, FText::FromName(ValidationPlatform.ShaderPlatformName)
					));
				}
			}
		}
	}

	if (!TranslationErrorMap.IsEmpty())
	{
		for (const TPair<FString, TArray<FName>>& ErrorAndPlatforms: TranslationErrorMap)
		{
			TStringBuilder<128> PlatformsString;
			for (const FName& Platform: ErrorAndPlatforms.Value)
			{
				if (PlatformsString.Len() != 0)
				{
					PlatformsString += TEXT(", ");
				}
				PlatformsString += Platform.ToString();
			}
			AssetFails(InAsset, FText::Format(
				LOCTEXT("MaterialValidator_TranslationError", "Failed to translate Material for platform(s) {0} due to '{1}'."),
				FText::FromStringView(PlatformsString.ToView()), FText::FromString(ErrorAndPlatforms.Key)
			));
		}
	}
	
	FMaterial::DeferredDeleteArray(TranslationResources);

	if (bTranslationFailed) // early out if translation failed
	{
		return EDataValidationResult::Invalid;
	}

	if (RetryWithCompilation.IsEmpty()) // assume validation is ok and early out
	{
		return EDataValidationResult::Valid;
	}

	// Right now calling CacheShaders multiple times on the same UMaterial(Instance) will not work as we don't clean up stale compilation jobs.
	// So to work around this we create a duplicate assets and compile cache shaders for them instead.
	UMaterialInstance* MaterialInstance = DuplicateMaterialInstance(OriginalMaterialInstance);
	UMaterial* Material = MaterialInstance ? MaterialInstance->GetMaterial() : DuplicateMaterial(OriginalMaterial);
	if (!ensureAlways(OriginalMaterial) || !ensureAlways(Material))
	{
		return EDataValidationResult::NotValidated;
	}

	TArray<FMaterialResource*> CompilationResources;
	TMap<FMaterialResource*, TArray<FName>> ResourceToShaderPlatformNames;

	for (const int32 ValidationPlatformIndex: RetryWithCompilation)
	{
		const FShaderValidationPlatform& ValidationPlatform = ValidationPlatforms[ValidationPlatformIndex];
		FMaterialResource* CurrentResource = FindOrCreateMaterialResource(CompilationResources, Material, MaterialInstance, ValidationPlatform.ShaderPlatform, ValidationPlatform.MaterialQualityLevel);

		if (ensure(CurrentResource))
		{
			CurrentResource->CacheShaders();
			ResourceToShaderPlatformNames.FindOrAdd(CurrentResource).Add(ValidationPlatform.ShaderPlatformName);
		}
	}

	if (CompilationResources.IsEmpty())
	{
		return EDataValidationResult::NotValidated;
	}

	FAssetCompilingManager::Get().FinishAllCompilation();

	bool bCompileErrors = false;
	for (FMaterialResource* Resource: CompilationResources)
	{
		if (!Resource->IsCompilationFinished())
		{
			UE_LOGF(LogContentValidation, Warning, "Shader compilation was expected to be finished, but was not finished.");
		}

		TArray<FName>* ShaderPlatformNames = ResourceToShaderPlatformNames.Find(Resource);
		const FString ShaderPlatformNameString = ShaderPlatformNames ? FString::JoinBy(*ShaderPlatformNames, TEXT(", "), [](const FName& Name) -> FString {return Name.ToString();}) : TEXT("Unknown");

		for (const FString& ErrorText: Resource->GetCompileErrors())
		{
			AssetFails(InAsset, FText::Format(
				LOCTEXT("MaterialValidator_CompilationError", "Failed to compile Material for platform {0} due to '{1}'."),
				FText::FromString(ShaderPlatformNameString), FText::FromString(ErrorText)
			));
			bCompileErrors = true;
		}
	}

	ResourceToShaderPlatformNames.Empty();

	FMaterial::DeferredDeleteArray(CompilationResources);

	return bCompileErrors ? EDataValidationResult::Invalid : EDataValidationResult::Valid;
}

UMaterial* UEditorValidator_Material::DuplicateMaterial(UMaterial* OriginalMaterial)
{
	if (!OriginalMaterial)
	{
		return nullptr;
	}

	return static_cast<UMaterial*>(StaticDuplicateObject(OriginalMaterial, GetTransientPackage(), NAME_None, ~RF_Standalone, UValidationMaterial::StaticClass()));
}

UMaterialInstance* UEditorValidator_Material::DuplicateMaterialInstance(UMaterialInstance* OriginalMaterialInstance)
{
	if (!OriginalMaterialInstance)
	{
		return nullptr;
	}

	TArray<UMaterialInstance*> DuplicatedMaterialInstances;

	FMaterialInheritanceChain Chain;
	OriginalMaterialInstance->GetMaterialInheritanceChain(Chain);

	for (const UMaterialInstance* MaterialInstance: Chain.MaterialInstances)
	{
		// only duplicate material instances that might influence compilation
		if (!MaterialInstance->HasStaticParameters())
		{
			continue;
		}

		UMaterialInstance* DuplicatedMaterialInstance = Cast<UMaterialInstance>(StaticDuplicateObject(MaterialInstance, GetTransientPackage(), NAME_None, ~RF_Standalone, MaterialInstance->GetClass()));

		DuplicatedMaterialInstances.Add(DuplicatedMaterialInstance);
	}

	// should be caught by CanValidateAsset_Implementation
	if (ensureAlways(DuplicatedMaterialInstances.Num() > 0))
	{
		UMaterial* DuplicatedMaterial = DuplicateMaterial(OriginalMaterialInstance->GetMaterial());

		for (int32 i = 0; i < DuplicatedMaterialInstances.Num(); ++i)
		{
			if (i + 1 < DuplicatedMaterialInstances.Num())
			{
				DuplicatedMaterialInstances[i]->Parent = DuplicatedMaterialInstances[i + 1];
			}
			else
			{
				DuplicatedMaterialInstances[i]->Parent = DuplicatedMaterial;
			}
		}

		return DuplicatedMaterialInstances[0];
	}
	else
	{
		return nullptr;
	}
}

class FValidationMaterial : public FMaterialResource
{
public:
	FValidationMaterial() = default;
	virtual ~FValidationMaterial() override = default;

	virtual bool IsPersistent() const override { return false; }
	virtual FString GetAssetName() const override { return FString::Printf(TEXT("Validation:%s"), *FMaterialResource::GetAssetName()); }
	virtual bool IsPreview() const override { return true; }
};

FMaterialResource* UValidationMaterial::AllocateResource()
{
	return new FValidationMaterial();
}

class FMaterialEditorValidationPlatformCustomization : public IPropertyTypeCustomization
{
public:
	FMaterialEditorValidationPlatformCustomization()
		: MaxRHIShaderPlatform(MakeShared<EShaderPlatform>(SP_NumPlatforms))
	{
		for (int32 ShaderPlatformIndex = 0; ShaderPlatformIndex < SP_NumPlatforms; ++ShaderPlatformIndex)
		{
			const EShaderPlatform ShaderPlatform = static_cast<EShaderPlatform>(ShaderPlatformIndex);

			if (FDataDrivenShaderPlatformInfo::IsValid(ShaderPlatform) && FDataDrivenShaderPlatformInfo::CanUseForMaterialValidation(ShaderPlatform))
			{
				ValidationShaderPlatforms.Add(MakeShared<EShaderPlatform>(ShaderPlatform));
			}
		}

		ValidationShaderPlatforms.StableSort([this](const TSharedPtr<EShaderPlatform>& A, const TSharedPtr<EShaderPlatform>& B)
		{
			return GetShaderPlatformFriendlyName(A).CompareTo(GetShaderPlatformFriendlyName(B)) < 0;
		});

		ValidationShaderPlatforms.Insert(MaxRHIShaderPlatform, 0);
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMaterialEditorValidationShaderPlatform, Name));
		ensure(PropertyHandle.IsValid());
		if (!PropertyHandle.IsValid())
		{
			return;
		}

		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboBox<TSharedPtr<EShaderPlatform>>)
			.OptionsSource(&ValidationShaderPlatforms)
			.InitiallySelectedItem(GetCurrentShaderPlatform(PropertyHandle))
			.OnSelectionChanged_Lambda([this, PropertyHandle](TSharedPtr<EShaderPlatform> ShaderPlatformOpt, ESelectInfo::Type SelectionInfo)
			{
				if (ShaderPlatformOpt.IsValid() && PropertyHandle.IsValid())
				{
					PropertyHandle->NotifyPreChange();
					if (ShaderPlatformOpt == MaxRHIShaderPlatform)
					{
						PropertyHandle->SetValue(FMaterialEditorValidationShaderPlatform::MaxRHIShaderPlatformName);
					}
					else
					{
						PropertyHandle->SetValue(FDataDrivenShaderPlatformInfo::GetName(*ShaderPlatformOpt.Get()));
					}
					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				}
			})
			.OnGenerateWidget_Lambda([this](const TSharedPtr<EShaderPlatform>& ShaderPlatformOpt)
			{
				return SNew(STextBlock).Text(GetShaderPlatformFriendlyName(ShaderPlatformOpt));
			})
			.Content()
			[
				SNew(STextBlock)
				.Font(StructCustomizationUtils.GetRegularFont())
				.Text_Lambda([this, PropertyHandle]()
				{
					return GetShaderPlatformFriendlyName(GetCurrentShaderPlatform(PropertyHandle)); 
				})
			]
		];
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
	}

private:
	TArray<TSharedPtr<EShaderPlatform>> ValidationShaderPlatforms;
	const TSharedPtr<EShaderPlatform> MaxRHIShaderPlatform;

	FName GetShaderPlatformName(const TSharedPtr<EShaderPlatform>& ShaderPlatformOpt)
	{
		if (ShaderPlatformOpt == MaxRHIShaderPlatform)
		{
			return FMaterialEditorValidationShaderPlatform::MaxRHIShaderPlatformName;
		}
		else if (ShaderPlatformOpt.IsValid())
		{
			return FDataDrivenShaderPlatformInfo::GetName(*ShaderPlatformOpt.Get());
		}
		else
		{
			return FName("Invalid");
		}
	}

	FText GetShaderPlatformFriendlyName(const TSharedPtr<EShaderPlatform>& ShaderPlatformOpt)
	{
		if (ShaderPlatformOpt == MaxRHIShaderPlatform)
		{
			return NSLOCTEXT("AssetValidation", "ShaderPlatform_MaxRHIShaderPlatform", "Current RHI Max Shader Platform");
		}
		else if (ShaderPlatformOpt.IsValid())
		{
			const FText Name = FDataDrivenShaderPlatformInfo::GetFriendlyName(*ShaderPlatformOpt.Get());
			if (!Name.IsEmpty())
			{
				return Name;
			}

			return FText::FromName(FDataDrivenShaderPlatformInfo::GetName(*ShaderPlatformOpt.Get()));
		}
		else
		{
			return NSLOCTEXT("AssetValidation", "ShaderPlatform_Invalid", "Invalid");
		}
	}

	TSharedPtr<EShaderPlatform> GetCurrentShaderPlatform(const TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		FName CurrentShaderPlatformName;
		if (PropertyHandle->GetValue(CurrentShaderPlatformName))
		{
			for (const TSharedPtr<EShaderPlatform>& ValidationShaderPlatform : ValidationShaderPlatforms)
			{
				if (GetShaderPlatformName(ValidationShaderPlatform) == CurrentShaderPlatformName)
				{
					return ValidationShaderPlatform;
				}
			}
		}

		return TSharedPtr<EShaderPlatform>();
	}
};

FName FMaterialEditorValidationShaderPlatform::MaxRHIShaderPlatformName = MaxRHIShaderPlatformNameView.GetData();
FName FMaterialEditorValidationShaderPlatform::CustomPropertyTypeLayoutName;

void FMaterialEditorValidationShaderPlatform::RegisterCustomPropertyTypeLayout()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	CustomPropertyTypeLayoutName = StaticStruct()->GetFName(); 
	PropertyModule.RegisterCustomPropertyTypeLayout(
		StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([](){return MakeShared<FMaterialEditorValidationPlatformCustomization>();})
	);
}

void FMaterialEditorValidationShaderPlatform::UnregisterCustomPropertyTypeLayout()
{
	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyModule)
	{
		// StaticStruct()->GetFName() is not available during engine shutdown as UObjects were already destroyed
		PropertyModule->UnregisterCustomPropertyTypeLayout(CustomPropertyTypeLayoutName);
	}
}

#undef LOCTEXT_NAMESPACE
