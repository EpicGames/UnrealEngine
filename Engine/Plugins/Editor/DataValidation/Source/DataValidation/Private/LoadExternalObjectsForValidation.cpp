// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoadExternalObjectsForValidation.h"

#include "AssetRegistry/AssetData.h"
#include "Engine/World.h"
#include "Misc/DataValidation.h"
#include "Misc/PathViews.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"

namespace UE::DataValidation
{

FScopedLoadExternalObjects::FScopedLoadExternalObjects(UObject* InAsset, FDataValidationContext& InContext, bool bEnabled)
{
    if (!bEnabled || InContext.GetAssociatedExternalObjects().Num() == 0)
    {
        return;
    }

    if (UWorld* World = Cast<UWorld>(InAsset); World && World->IsPartitionedWorld())
    {
        auto RegisterContainerToValidate = [this](UWorld* InWorld, FName InContainerPackageName, FActorDescContainerInstanceCollection& OutRegisteredContainers, const FGuid& InContentBundleGuid = FGuid(), const UExternalDataLayerAsset* InExternalDataLayerAsset = nullptr)
        {
            if (OutRegisteredContainers.Contains(InContainerPackageName))
            {
                return;
            }

            UActorDescContainerInstance* ContainerInstance = nullptr;
            if (InWorld != nullptr)
            {
                // World is Loaded reuse the ActorDescContainer of the Content Bundle
                ContainerInstance = InWorld->GetWorldPartition()->FindContainer(InContainerPackageName);
            }

            // Even if world is valid, its world partition is not necessarily initialized
            if (!ContainerInstance)
            {
                // Find in memory failed, load the ActorDescContainer
                ContainerInstance = NewObject<UActorDescContainerInstance>();
                UActorDescContainerInstance::FInitializeParams InitializeParams(InContainerPackageName);
                InitializeParams.ContentBundleGuid = InContentBundleGuid;
                InitializeParams.ExternalDataLayerAsset = InExternalDataLayerAsset;
                ContainerInstance->Initialize(InitializeParams);
                ContainersToUninit.Emplace(ContainerInstance);
            }
            else
            {
                check(ContainerInstance->GetContentBundleGuid() == InContentBundleGuid);
                check(ContainerInstance->GetExternalDataLayerAsset() == InExternalDataLayerAsset);
            }

            OutRegisteredContainers.AddContainer(ContainerInstance);
        };

        TSet<FSoftObjectPath> ProcessedExternalDataLayers;
        FString MapPackageName = World->GetPackage()->GetName();
        TConstArrayView<FAssetData> ActorsData = InContext.GetAssociatedExternalObjects();
        FActorDescContainerInstanceCollection ContainersToValidate;
        for (const FAssetData& ActorData : ActorsData)
        {
            FString ActorPackagePath = ActorData.PackagePath.ToString();
            if (ContentBundlePaths::IsAContentBundleExternalActorPackagePath(ActorPackagePath))
            {
                FStringView ContentBundleMountPoint = FPathViews::GetMountPointNameFromPath(ActorPackagePath);
                FGuid ContentBundleGuid = ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(ActorPackagePath);
                
                FString ContentBundleContainerPackagePath;
                verify(ContentBundlePaths::BuildActorDescContainerPackagePath(FString(ContentBundleMountPoint), ContentBundleGuid, MapPackageName, ContentBundleContainerPackagePath));

                RegisterContainerToValidate(World, FName(*ContentBundleContainerPackagePath), ContainersToValidate, ContentBundleGuid);
            }
            else
            {
                if (TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(ActorData))
                {
                    bool bIsAlreadyInSet = false;
                    const FSoftObjectPath& ExternalDataLayerPath = ActorDesc->GetExternalDataLayerAsset();
                    if (ExternalDataLayerPath.IsValid())
                    {
                        ProcessedExternalDataLayers.Add(ExternalDataLayerPath, &bIsAlreadyInSet);
                        if (!bIsAlreadyInSet)
                        {
                            if (const UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(ExternalDataLayerPath.TryLoad()))
                            {
                                const FString EDLContainerPackagePath = FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(ExternalDataLayerAsset, MapPackageName);
                                RegisterContainerToValidate(World, FName(*EDLContainerPackagePath), ContainersToValidate, FGuid(), ExternalDataLayerAsset);
                            }
                        }
                    }
                    else
                    {
                        RegisterContainerToValidate(World, World->GetPackage()->GetFName(), ContainersToValidate);
                    }
                }
            }
        }
        
		{
			FWorldPartitionLoadingContext::FDeferred LoadingContext;
			for (const FAssetData& ActorData : ActorsData)
			{
				if (const FWorldPartitionActorDescInstance* ActorDesc = ContainersToValidate.GetActorDescInstanceByName(ActorData.AssetName))
				{
					ActorRefs.Add(FWorldPartitionReference(&ContainersToValidate, ActorDesc->GetGuid()));
				}
			}
		}
    }
}

FScopedLoadExternalObjects::~FScopedLoadExternalObjects()
{
    // Explicitly release actors before uninitializing containers
	{
		FWorldPartitionLoadingContext::FDeferred LoadingContext;
		ActorRefs.Empty();
	}

	for (const TStrongObjectPtr<UActorDescContainerInstance>& ContainerToUninit : ContainersToUninit)
	{
		ContainerToUninit->Uninitialize();
	}
    ContainersToUninit.Empty();
}
}