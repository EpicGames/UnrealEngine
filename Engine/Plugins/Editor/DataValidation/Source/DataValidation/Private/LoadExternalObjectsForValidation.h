// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtrTemplates.h"
#include "WorldPartition/WorldPartitionHandle.h"

class FDataValidationContext;
class UActorDescContainerInstance;
class UObject;

namespace UE::DataValidation
{
    // Utility to load external objects (e.g. actors) associated with an asset (e.g. a map) for validation so that they are 
    // available to validators inspecting the main asset. 
    // Should only be used on the stack. 
    struct FScopedLoadExternalObjects
    {
        FScopedLoadExternalObjects(UObject* InAsset, FDataValidationContext& InContext, bool bEnabled);
        ~FScopedLoadExternalObjects();
 
    private:
        // Hard refs keeping actors loaded for the same lifetime as this object
        TArray<FWorldPartitionReference> ActorRefs;
        // Containers used for actor loading which must be explicitly uninitialized at the end of this object's lifetime
        TArray<TStrongObjectPtr<UActorDescContainerInstance>> ContainersToUninit;
    };
}