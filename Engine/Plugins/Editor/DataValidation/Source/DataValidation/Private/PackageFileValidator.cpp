// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageFileValidator.h"

#include "DataValidationChangelist.h"
#include "EditorValidatorSubsystem.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "Misc/DataValidation.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/PackageTrailer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PackageFileValidator)

#define LOCTEXT_NAMESPACE "PackageFileValidator"

bool UPackageFileValidator::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* Asset, FDataValidationContext& Context) const
{
	// We don't want to validate the package on disk when saving, as we will be overwriting that file anyway
	if (Context.GetValidationUsecase() == EDataValidationUsecase::Save)
	{
		return false;
	}

	if (Asset == nullptr)
	{
		return false;
	}

	// Assets should always be part of a package but we should check to be sure
	UPackage* Package = Asset->GetPackage();
	if (Package == nullptr)
	{
		return false;
	}

	// The package will need a valid name if we are to find it's file on disk
	if (Package->GetFName().IsNone())
	{
		return false;
	}

	// Avoid in memory and/or transient packages as they won't exist on disk
	if (Package->HasAnyPackageFlags(PKG_InMemoryOnly) || Package->HasAnyPackageFlags(PKG_TransientFlags) || Package == GetTransientPackage())
	{
		return false;
	}

	// See if we can resolve the package name to a valid package path. This might fail if the package mount point is disabled or if
	// the package does not have a file on disk yet.
	FPackagePath PackagePath;
	if (!TryResolvePackagePath(Package->GetFName(), PackagePath))
	{
		return false;
	}

	return true;
}

EDataValidationResult UPackageFileValidator::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* Asset, FDataValidationContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPackageFileValidator::ValidateLoadedAsset_Implementation);

	check(Asset->GetPackage()); // Was already validated in CanValidateAsset_Implementation
	const FName PackageName = Asset->GetPackage()->GetFName();

	FPackagePath PackagePath;
	ensure(TryResolvePackagePath(PackageName, PackagePath)); // Was already validated in CanValidateAsset_Implementation

	TUniquePtr<FArchive> PackageAr(IFileManager::Get().CreateFileReader(*PackagePath.GetLocalFullPath(), FILEREAD_Silent));
	if (!PackageAr.IsValid())
	{
		AssetFails(nullptr, FText::Format(LOCTEXT("FailedPkgOpen", "{0} Unable to open for reading"), FText::FromName(PackageName)));
		return EDataValidationResult::Invalid;
	}

	FPackageFileSummary Summary;
	if (!ValidatePackageSummary(PackageName, *PackageAr, Context, Summary))
	{
		return EDataValidationResult::Invalid;
	}

	if (Summary.PayloadTocOffset > 0 && !ValidatePackageTrailer(PackageName, *PackageAr, Context))
	{
		return EDataValidationResult::Invalid;
	}

	return EDataValidationResult::Valid;
}

bool UPackageFileValidator::ValidatePackageSummary(FName PackageName, FArchive& Ar, FDataValidationContext& Context, FPackageFileSummary& OutSummary)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPackageFileValidator::ValidatePackageSummary);

	const int64 TagOffset = Ar.TotalSize() - sizeof(uint32);
	Ar.Seek(TagOffset);

	uint32 Tag = 0;
	Ar << Tag;

	if (Tag != PACKAGE_FILE_TAG || Ar.IsError())
	{
		AssetFails(nullptr, FText::Format(LOCTEXT("BadPkgTag", "{0} The end of package tag is not valid, the file is probably corrupt"), FText::FromName(PackageName)));
		return false;
	}

	Ar.Seek(0);

	Ar << OutSummary;

	if (Ar.IsError() || OutSummary.Tag != PACKAGE_FILE_TAG)
	{
		AssetFails(nullptr, FText::Format(LOCTEXT("BadPkgSummary", "{0} Failed to read the package file summary, the file is probably corrupt"), FText::FromName(PackageName)));
		return false;
	}

	if (OutSummary.IsFileVersionTooOld())
	{
		AssetFails(nullptr, FText::Format(LOCTEXT("PkgOutOfDate", "{0} is out of date and is not backwards compatible with the current process. Min Required Version: {1}  Package Version: {2}"),
			FText::FromName(PackageName),
			(int32)VER_UE4_OLDEST_LOADABLE_PACKAGE,
			OutSummary.GetFileVersionUE().FileVersionUE4));
		return false;
	}

	return true;
}

bool UPackageFileValidator::ValidatePackageTrailer(FName PackageName, FArchive& Ar, FDataValidationContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPackageFileValidator::ValidatePackageTrailer);

	using namespace UE;

	FPackageTrailer Trailer;
	if (!FPackageTrailer::TryLoadFromArchive(Ar, Trailer))
	{
		AssetFails(nullptr, FText::Format(LOCTEXT("BadPkgTrailer", "{0} Failed to read the package trailer, the file is probably corrupt"), FText::FromName(PackageName)));
		return false;
	}

	TArray<FIoHash> LocalPayloads = Trailer.GetPayloads(EPayloadStorageType::Local);

	for (const FIoHash& Id : LocalPayloads)
	{
		FCompressedBuffer Payload = Trailer.LoadLocalPayload(Id, Ar);
		if (Payload.IsNull())
		{
			AssetFails(nullptr, FText::Format(LOCTEXT("BadPayload", "{0} Failed to read the payload {1}, the file is probably corrupt"), FText::FromName(PackageName), FText::FromString(LexToString(Id))));
			return false;
		}

		if (Id != Payload.GetRawHash())
		{
			AssetFails(nullptr, FText::Format(LOCTEXT("BadPayloadId", "{0} Failed to read the payload {1}, the file is probably corrupt"), FText::FromName(PackageName), FText::FromString(LexToString(Id))));
			return false;
		}

		if (bValidatePayloadHashes)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UPackageFileValidator::ValidatePackageTrailer::HashPayload);

			FIoHash PayloadHash = FIoHash::HashBuffer(Payload.Decompress());
			if (Id != PayloadHash)
			{
				AssetFails(nullptr, FText::Format(LOCTEXT("BadPayloadData", "{0} The payload data did not match it's stored hash {1} vs {2}, the file is probably corrupt"),
					FText::FromName(PackageName),
					FText::FromString(LexToString(Id)),
					FText::FromString(LexToString(PayloadHash))));

				return false;
			}
		}
	}

	return true;
}

bool UPackageFileValidator::TryResolvePackagePath(FName PackageName, FPackagePath& OutPackagePath) const
{
	FPackagePath PackagePath;
	if (!FPackagePath::TryFromPackageName(PackageName, PackagePath))
	{
		return false;
	}

	if (!IPackageResourceManager::Get().DoesPackageExist(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header, &OutPackagePath))
	{
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
