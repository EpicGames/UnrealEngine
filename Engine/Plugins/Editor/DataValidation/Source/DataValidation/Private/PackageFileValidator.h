// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"

#include "PackageFileValidator.generated.h"

#define UE_API DATAVALIDATION_API

struct FPackageFileSummary;

/**
 * This validator checks the format of the package on disk to make sure that is has not become corrupted since it was last saved.
 * 
 * To disable the validator entirely, set ini:Editor:[/Script/DataValidation.PackageFileValidator]:bIsConfigDisabled=true
 * To disable validation of payload hashes (which is much slower than the rest of the validation) set 
 * ini:Editor:[/Script/DataValidation.PackageFileValidator]:bValidatePayloadHashes=true
 */
UCLASS(MinimalAPI)
class UPackageFileValidator : public UEditorValidatorBase
{
	GENERATED_BODY()

protected:

	UE_API virtual bool CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* Asset, FDataValidationContext& Context) const override;
	UE_API virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* Asset, FDataValidationContext& Context) override;

private:

	UE_API bool ValidatePackageSummary(FName PackageName, FArchive& Ar, FDataValidationContext& Context, FPackageFileSummary& OutSummary);
	UE_API bool ValidatePackageTrailer(FName PackageName, FArchive& Ar, FDataValidationContext& Context);

	UE_API bool TryResolvePackagePath(FName PackageName, FPackagePath& OutPackagePath) const;

	UPROPERTY(Config)
	bool bValidatePayloadHashes = true;
};

#undef UE_API
