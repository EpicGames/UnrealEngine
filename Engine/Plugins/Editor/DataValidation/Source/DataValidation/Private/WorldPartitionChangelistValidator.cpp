// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionChangelistValidator.h"
#include "AssetRegistry/ARFilter.h"
#include "DataValidationChangelist.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor.h"
#include "EditorValidatorSubsystem.h"
#include "ISourceControlModule.h"
#include "Misc/PackageName.h"
#include "SourceControlHelpers.h"

#include "Engine/Level.h"
#include "Misc/PathViews.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayerInstanceWithAsset.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionChangelistValidator)

#define LOCTEXT_NAMESPACE "WorldPartitionChangelistValidation"

bool UWorldPartitionChangelistValidator::CanValidateAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext) const
{
	return (InAsset != nullptr) && (UDataValidationChangelist::StaticClass() == InAsset->GetClass());
}

EDataValidationResult UWorldPartitionChangelistValidator::ValidateLoadedAsset_Implementation(const FAssetData& AssetData, UObject* InAsset, FDataValidationContext& InContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionChangelistValidator::ValidateLoadedAsset_Implementation);

	UDataValidationChangelist* ChangeList = CastChecked<UDataValidationChangelist>(InAsset);
	
	ValidateActorsAndDataLayersFromChangeList(ChangeList);
	switch (GetValidationResult())
	{
		case EDataValidationResult::Invalid:
			AssetFails(InAsset, LOCTEXT("WorldPartitionValidationFail", 
				"This changelist contains modifications that aren't valid at the world partition level. Please see the message log for the errors preceding this message."));
			break;
		case EDataValidationResult::Valid:
		case EDataValidationResult::NotValidated:
			AssetPasses(InAsset);
			break;
	}

	return GetValidationResult();
}

// Extract all Actors/Map from Changelist (in OFPA this should be one Actor per Package, and we'll discard all Actors from non WorldPartition maps)
// and add them to a Map of World->Files[] so that we can do one validation per world. Once Worlds are identified, we either the UActorDescContainerInstance 
// from memory (if loaded) or request it to be loaded, we then build a Set of objects that interest us from the Actors in the CL 
void UWorldPartitionChangelistValidator::ValidateActorsAndDataLayersFromChangeList(UDataValidationChangelist* Changelist)
{
	// Figure out which world(s) these actors are in and split the files per world, and return the actor's native class
	TMap<FTopLevelAssetPath, TSet<FAssetData>> MapToActorsFiles;
	auto TryAssociateActorToMap = [&MapToActorsFiles](const FAssetData& AssetData) -> UClass*
	{
		// Check that the asset is an actor
		if (FWorldPartitionActorDescUtils::IsValidActorDescriptorFromAssetData(AssetData))
		{
			// WorldPartition actors are all in OFPA mode so they're external
			// Extract the MapName from the ObjectPath (<PathToPackage>.<MapName>:<Level>.<ActorName>)
			FSoftObjectPath ActorPath = AssetData.GetSoftObjectPath();
			FTopLevelAssetPath MapAssetName = ActorPath.GetAssetPath();

			TSet<FAssetData>* ActorFiles = MapToActorsFiles.Find(MapAssetName);

			if (!ActorFiles)
			{
				FName PackageName = ActorPath.GetLongPackageFName();
				if (FPackageName::IsValidLongPackageName(PackageName.ToString()))
				{
					if (ULevel::GetIsLevelPartitionedFromPackage(PackageName))
					{
						ActorFiles = &MapToActorsFiles.Add(MapAssetName);
					}
				}
				else
				{
					UE_LOGF(LogContentValidation, Log, "ValidateActorsAndDataLayersFromChangeList found a map with invalid path '%ls'. It may be in an umounted content folder. Referencing actor package: %ls", *PackageName.ToString(), *AssetData.PackageName.ToString());
				}
			}

			if (ActorFiles)	// A null Files indicates a World not using World Partition and OFPA 
			{
				ActorFiles->Add(AssetData);
				return FWorldPartitionActorDescUtils::GetActorNativeClassFromAssetData(AssetData);
			}
		}

		return nullptr;
	};

	UEditorValidatorSubsystem* EditorValidationSubsystem = GEditor ? GEditor->GetEditorSubsystem<UEditorValidatorSubsystem>() : nullptr;
	FValidateAssetsSettings Settings;
	FDataValidationContext ValidationContext(false, Settings.ValidationUsecase, {});
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	for (FName PackageName : Changelist->ModifiedPackageNames)
	{		
		TArray<FAssetData> PackageAssetsData;
		USourceControlHelpers::GetAssetDataFromPackage(PackageName.ToString(), PackageAssetsData);
		
		for (FAssetData& AssetData : PackageAssetsData)
		{
			if (!EditorValidationSubsystem || EditorValidationSubsystem->ShouldValidateAsset(AssetData, Settings, ValidationContext))
			{
				if (UClass* ActorNativeClass = TryAssociateActorToMap(AssetData))
				{
					bSubmittingWorldDataLayers = ActorNativeClass->IsChildOf<AWorldDataLayers>();
				}
				else if (UClass* AssetClass = AssetData.GetClass())
				{
					if (AssetClass->IsChildOf<UDataLayerAsset>())
					{
						TArray<FName> ReferencerNames;
						AssetRegistry.GetReferencers(AssetData.PackageName, ReferencerNames, UE::AssetRegistry::EDependencyCategory::All);

						FARFilter Filter;
						Filter.bIncludeOnlyOnDiskAssets = true;
						Filter.PackageNames = MoveTemp(ReferencerNames);

						TArray<FAssetData> DataLayerReferencers;
						AssetRegistry.GetAssets(Filter, DataLayerReferencers);

						for (const FAssetData& DataLayerReferencer : DataLayerReferencers)
						{
							UClass* ReferencerAssetClass = DataLayerReferencer.GetClass();
							if (ReferencerAssetClass && ReferencerAssetClass->IsChildOf<AWorldDataLayers>())
							{
								TryAssociateActorToMap(DataLayerReferencer);
							}
						}

						RelevantDataLayerAssets.Add(AssetData.PackageName.ToString());
					}
					else if (AssetClass->IsChildOf<UDataLayerInstance>())
					{
						RelevantExternalPackageDataLayerInstances.Add(AssetData.PackageName.ToString());
					}
					else if (AssetClass->IsChildOf<UWorld>())
					{
						if (ULevel::GetIsLevelPartitionedFromPackage(PackageName))
						{
							MapToActorsFiles.FindOrAdd(AssetData.GetSoftObjectPath().GetAssetPath());
						}
					}
				}
			}
		}
	}

	auto RegisterContainerToValidate = [](UWorld* InWorld, FName InContainerPackageName, FActorDescContainerInstanceCollection& OutRegisteredContainers, TArray<UActorDescContainerInstance*>& OutNewlyCreatedContainers, const FGuid& InContentBundleGuid = FGuid(), const UExternalDataLayerAsset* InExternalDataLayerAsset = nullptr)
	{
		if (OutRegisteredContainers.Contains(InContainerPackageName))
		{
			return;
		}

		UActorDescContainerInstance* ContainerInstance = nullptr;
		if (InWorld != nullptr && InWorld->GetWorldPartition()->IsInitialized() && InWorld->GetWorldPartition()->IsMainWorldPartition())
		{
			// World is Loaded reuse the ActorDescContainer (Can be Main world package or one of the Content Bundle packages associated with main world)
			// Only reuse Main world partiton worlds because streaming generation expects the top level container to always be a Main container.
			// It is possible that InWorld is a LevelInstance edit world in which case we don't want to use its loaded ContainerInstance and instead create a new one
			ContainerInstance = InWorld->GetWorldPartition()->FindContainer(InContainerPackageName);
		}

		// Even if world is valid, its world partition is not necessarily initialized
		if (!ContainerInstance)
		{
			// Find in memory failed, load the ActorDescContainerInstance
			ContainerInstance = NewObject<UActorDescContainerInstance>();
			UActorDescContainerInstance::FInitializeParams InitializeParams(InContainerPackageName);
			InitializeParams.ContentBundleGuid = InContentBundleGuid;
			InitializeParams.ExternalDataLayerAsset = InExternalDataLayerAsset;
			ContainerInstance->Initialize(InitializeParams);
			OutNewlyCreatedContainers.Add(ContainerInstance);
		}
		else
		{
			check(ContainerInstance->GetContentBundleGuid() == InContentBundleGuid);
			check(ContainerInstance->GetExternalDataLayerAsset() == InExternalDataLayerAsset);
		}

		OutRegisteredContainers.AddContainer(ContainerInstance);
	};

	// For Each world 
	for (TTuple<FTopLevelAssetPath, TSet<FAssetData>>& It : MapToActorsFiles)
	{
		const FTopLevelAssetPath& MapPath = It.Get<0>();
		const TSet<FAssetData>& ActorsData = It.Get<1>();

		// Find/Load the ActorDescContainer
		UWorld* World = FindObject<UWorld>(nullptr, *MapPath.ToString(), EFindObjectFlags::ExactClass);
		
		TGuardValue<UObject*> GuardCurrentAsset(CurrentAsset, World);
		
		FActorDescContainerInstanceCollection ContainersToValidate;
		TArray<UActorDescContainerInstance*> ContainersToUninit;

		// Always register the main world container because content bundle containers can't be validated separately
		RegisterContainerToValidate(World, MapPath.GetPackageName(), ContainersToValidate, ContainersToUninit);

		TSet<FSoftObjectPath> ProcessedExternalDataLayersForMap;
		for (const FAssetData& ActorData : ActorsData)
		{
			FString ActorPackagePath = ActorData.PackagePath.ToString();
			if (ContentBundlePaths::IsAContentBundleExternalActorPackagePath(ActorPackagePath))
			{
				FStringView ContentBundleMountPoint = FPathViews::GetMountPointNameFromPath(ActorPackagePath);
				FGuid ContentBundleGuid = ContentBundlePaths::GetContentBundleGuidFromExternalActorPackagePath(ActorPackagePath);
				
				FString ContentBundleContainerPackagePath;
				verify(ContentBundlePaths::BuildActorDescContainerPackagePath(FString(ContentBundleMountPoint), ContentBundleGuid, MapPath.GetPackageName().ToString(), ContentBundleContainerPackagePath));

				RegisterContainerToValidate(World, FName(*ContentBundleContainerPackagePath), ContainersToValidate, ContainersToUninit, ContentBundleGuid);
			}
			else
			{
				if (TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(ActorData))
				{
					bool bIsAlreadyInSet = false;
					const FSoftObjectPath& ExternalDataLayerPath = ActorDesc->GetExternalDataLayerAsset();
					if (ExternalDataLayerPath.IsValid())
					{
						ProcessedExternalDataLayersForMap.Add(ExternalDataLayerPath, &bIsAlreadyInSet);
						if (!bIsAlreadyInSet)
						{
							if (const UExternalDataLayerAsset* ExternalDataLayerAsset = Cast<UExternalDataLayerAsset>(ExternalDataLayerPath.TryLoad()))
							{
								const FString EDLContainerPackagePath = FExternalDataLayerHelper::GetExternalDataLayerLevelRootPath(ExternalDataLayerAsset, MapPath.GetPackageName().ToString());
								RegisterContainerToValidate(World, FName(*EDLContainerPackagePath), ContainersToValidate, ContainersToUninit, FGuid(), ExternalDataLayerAsset);
							}
						}
					}
				}
			}
		}

		// Build a set of Relevant Actor Guids to scope error messages to what's contained in the CL 
		RelevantMap = MapPath;
		RelevantActorGuids.Reset();

		for (const FAssetData& ActorData : ActorsData)
		{
			// Get the actor descriptor
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = ContainersToValidate.GetActorDescInstanceByName(ActorData.AssetName))
			{
				RelevantActorGuids.Add(ActorDescInstance->GetGuid());
			}
		}

		// Invoke static WorldPartition Validation from the ActorDescContainer
		UWorldPartition::FCheckForErrorsParams Params = UWorldPartition::FCheckForErrorsParams()
			.SetErrorHandler(this)
			.SetEnableStreaming(!ULevel::GetIsStreamingDisabledFromPackage(MapPath.GetPackageName()));

		Params.ActorDescContainerInstanceCollection = &ContainersToValidate;
		UWorldPartition::CheckForErrors(Params);

		for (UActorDescContainerInstance* ContainerToUninit : ContainersToUninit)
		{
			ContainerToUninit->Uninitialize();
		}
	}
}

bool UWorldPartitionChangelistValidator::Filter(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	if (RelevantActorGuids.Find(ActorDescView.GetGuid()))
	{
		return true;
	}

	if (!RelevantMap.IsNull())
	{
		FSoftObjectPath ActorPath = ActorDescView.GetActorSoftPath();
		return ActorPath.GetAssetPath() == RelevantMap;
	}

	return false;
}

bool UWorldPartitionChangelistValidator::Filter(const UDataLayerInstance* InDataLayerInstance)
{
	const UDataLayerInstanceWithAsset* DataLayerWithAsset = Cast<UDataLayerInstanceWithAsset>(InDataLayerInstance);
	if (DataLayerWithAsset && DataLayerWithAsset->GetAsset() && RelevantDataLayerAssets.Contains(DataLayerWithAsset->GetAsset()->GetPackage()->GetName()))
	{
		return true;
	}

	if (InDataLayerInstance->IsPackageExternal() && RelevantExternalPackageDataLayerInstances.Contains(InDataLayerInstance->GetPackage()->GetName()))
	{
		return true;
	}

	return false;
}

void UWorldPartitionChangelistValidator::OnInvalidRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, FName GridName)
{
	if (Filter(ActorDescView))
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidRuntimeGrid", "Actor {0} has an invalid runtime grid {1}"),
											FText::FromString(GetFullActorName(ActorDescView)), 
											FText::FromName(GridName));

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnInvalidReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const FGuid& ReferenceGuid, IWorldPartitionActorDescInstanceView* ReferenceActorDescView)
{
	if (Filter(ActorDescView))
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidReference", "Actor {0} has an invalid reference to {1}"),
											FText::FromString(GetFullActorName(ActorDescView)), 
											FText::FromString(ReferenceActorDescView ? GetFullActorName(*ReferenceActorDescView) : ReferenceGuid.ToString()));

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnInvalidReferenceGridPlacement(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	if (Filter(ActorDescView) || Filter(ReferenceActorDescView))
	{
		// Only report errors for non-spatially loaded actor referencing a spatially loaded actor
		if (!ActorDescView.GetIsSpatiallyLoaded())
		{
			const FString SpatiallyLoadedActor(TEXT("Spatially loaded actor"));
			const FString NonSpatiallyLoadedActor(TEXT("Non-spatially loaded actor"));

			FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidReferenceGridPlacement", "{0} {1} is referencing {2} {3}."),
												FText::FromString(ActorDescView.GetIsSpatiallyLoaded() ? *SpatiallyLoadedActor : *NonSpatiallyLoadedActor),
												FText::FromString(GetFullActorName(ActorDescView)),
												FText::FromString(ReferenceActorDescView.GetIsSpatiallyLoaded() ? *SpatiallyLoadedActor : *NonSpatiallyLoadedActor),
												FText::FromString(GetFullActorName(ReferenceActorDescView)));

			AssetFails(CurrentAsset, CurrentError);
		}
	}
}

void UWorldPartitionChangelistValidator::OnInvalidReferenceDataLayers(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView, EDataLayerInvalidReason Reason)
{	
	if (Filter(ActorDescView) || Filter(ReferenceActorDescView))
	{
		FText CurrentError;
		switch (Reason)
		{
		case EDataLayerInvalidReason::ReferencedActorDifferentRuntimeDataLayers:
			CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.DataLayerError", "{0} is referencing {1} but both actors are using a different set of runtime data layers."),
				FText::FromString(GetFullActorName(ActorDescView)),
				FText::FromString(GetFullActorName(ReferenceActorDescView)));
			break;
		case EDataLayerInvalidReason::ReferencedActorDifferentExternalDataLayer:
			CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.ExternalDataLayerError", "{0} is referencing {1} but both actors are assigned to a different external data layer."),
				FText::FromString(GetFullActorName(ActorDescView)),
				FText::FromString(GetFullActorName(ReferenceActorDescView)));
			break;
		}

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnInvalidReferenceRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	if (Filter(ActorDescView) || Filter(ReferenceActorDescView))
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.RuntimeGridError", "{0} is referencing {1} but both actors are using a different runtime grid."),
			FText::FromString(GetFullActorName(ActorDescView)),
			FText::FromString(GetFullActorName(ReferenceActorDescView)));

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnDataLayersLoadFilterMismatch(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	if (Filter(ActorDescView))
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.DataLayersLoadFilterMismatchError", "{0} contains runtime data layers with different types of Load Filter"),
			FText::FromString(GetFullActorName(ActorDescView)));

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnInvalidWorldReference(const IWorldPartitionActorDescInstanceView& ActorDescView, EWorldReferenceInvalidReason Reason)
{
	if (Filter(ActorDescView))
	{
		FText CurrentError;

		switch(Reason)
		{
		case EWorldReferenceInvalidReason::ReferencedActorIsSpatiallyLoaded:
			CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidWorldReferenceSpatiallyLoaded", "World references spatially loaded actor {0}."), FText::FromString(GetFullActorName(ActorDescView)));
			break;
		case EWorldReferenceInvalidReason::ReferencedActorHasDataLayers:
			CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidWorldReferenceDataLayers", "World references actor {0} with data layers."), FText::FromString(GetFullActorName(ActorDescView)));
			break;
		}
		
		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnInvalidReferenceDataLayerAsset(const UDataLayerInstanceWithAsset* DataLayerInstance)
{
	if (bSubmittingWorldDataLayers)
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidDataLayerAsset", "Data layer {0} has no data layer asset."),
			FText::FromName(DataLayerInstance->GetFName()));

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnInvalidDataLayerAssetType(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerAsset* DataLayerAsset)
{
	if (bSubmittingWorldDataLayers)
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidDataLayerAssetAsset", "Data layer {0} is not compatible with Data Layer Asset {1} type {2}."),
			FText::FromName(DataLayerInstance->GetFName()), FText::FromName(DataLayerAsset->GetFName()), FText::FromName(DataLayerAsset->GetClass()->GetFName()));

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnDataLayerHierarchyTypeMismatch(const UDataLayerInstance* DataLayerInstance, const UDataLayerInstance* Parent, EDataLayerHierarchyInvalidReason Reason)
{
	if (Filter(DataLayerInstance)
		|| Filter(Parent)
		|| bSubmittingWorldDataLayers)
	{
		switch (Reason)
		{
		case EDataLayerHierarchyInvalidReason::ClientOnlyDataLayerCantBeChild:
		case EDataLayerHierarchyInvalidReason::ServerOnlyDataLayerCantBeChild:
			{
				FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.DataLayerHierarchyCantBeChild", "Data layer {0} is {1} which can't be child of parent {2}."),
					FText::FromString(DataLayerInstance->GetDataLayerFullName()),
					(Reason == EDataLayerHierarchyInvalidReason::ClientOnlyDataLayerCantBeChild) ? LOCTEXT("DataLayerIsClientOnly", "client-only") : LOCTEXT("DataLayerIsServerOnly", "server-only"),
					FText::FromString(Parent->GetDataLayerFullName()));
				AssetFails(CurrentAsset, CurrentError);
			}
			break;
		case EDataLayerHierarchyInvalidReason::IncompatibleDataLayerType:
			{
				FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.DataLayerHierarchyTypeMismatch", "Data layer {0} is of type {1} and its parent {2} is of type {3}."),
					FText::FromString(DataLayerInstance->GetDataLayerFullName()),
					UEnum::GetDisplayValueAsText(DataLayerInstance->GetType()),
					FText::FromString(Parent->GetDataLayerFullName()),
					UEnum::GetDisplayValueAsText(Parent->GetType()));
				AssetFails(CurrentAsset, CurrentError);
			}
			break;
		}
	}
}

void UWorldPartitionChangelistValidator::OnInvalidWorldDataLayersReference(const AWorldDataLayers* WorldDataLayers, const UDataLayerInstance* DataLayerInstance, const FText& Reason)
{
	if (RelevantActorGuids.Find(WorldDataLayers->GetActorGuid()) || Filter(DataLayerInstance))
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.DataLayerAssetReferenceRestrictions", "Actor {0} can't reference data layer {1} because of asset reference restrictions ({2})"),
			FText::FromString(WorldDataLayers->GetName()),
			FText::FromString(DataLayerInstance->GetDataLayerFullName()),
			Reason);

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnDataLayerAssetConflict(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerInstanceWithAsset* ConflictingDataLayerInstance)
{
	if (Filter(DataLayerInstance)
		|| Filter(ConflictingDataLayerInstance)
		|| bSubmittingWorldDataLayers)
	{
		FText CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.DataLayerAssetConflict", "Data layer instance {0} and data layer instance {1} are both referencing data layer asset {2}."),
			FText::FromName(DataLayerInstance->GetFName()),
			FText::FromName(ConflictingDataLayerInstance->GetFName()),
			FText::FromString(DataLayerInstance->GetAsset()->GetFullName()));

		AssetFails(CurrentAsset, CurrentError);
	}
}

void UWorldPartitionChangelistValidator::OnActorNeedsResave(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	// Changelist validation already ensures that dirty actors must be part of the changelist
}

void UWorldPartitionChangelistValidator::OnLevelInstanceInvalidWorldAsset(const IWorldPartitionActorDescInstanceView& ActorDescView, FName WorldAsset, ELevelInstanceInvalidReason Reason)
{
	if (Filter(ActorDescView))
	{
		FText CurrentError;

		switch (Reason)
		{
		case ELevelInstanceInvalidReason::WorldAssetNotFound:
			CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.LevelInstanceInvalidWorldAsset", "Level instance {0} has an invalid world asset {1}."),
				FText::FromString(GetFullActorName(ActorDescView)), 
				FText::FromName(WorldAsset));
			AssetFails(CurrentAsset, CurrentError);
			break;
		case ELevelInstanceInvalidReason::WorldAssetDontContainActorsMetadata:
			// Not a validation error
			break;
		case ELevelInstanceInvalidReason::WorldAssetIncompatiblePartitioned:
			// Not a validation error
			break;
		case ELevelInstanceInvalidReason::WorldAssetHasInvalidContainer:
			// We cannot treat that error as a validation error as it's possible to validate changelists without loading the world
			break;
		case ELevelInstanceInvalidReason::CirculalReference:
			CurrentError = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.LevelInstanceCircularReference", "Level instance {0} has a circular reference {1}."),
				FText::FromString(GetFullActorName(ActorDescView)),
				FText::FromName(WorldAsset));
			AssetFails(CurrentAsset, CurrentError);
			break;
		};
	}
}

void UWorldPartitionChangelistValidator::OnInvalidActorFilterReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView)
{
	// Not a validation error
}

void UWorldPartitionChangelistValidator::OnInvalidHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView)
{
	if (Filter(ActorDescView))
	{
		FText CurrentWarning = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.InvalidHLODLayer", "Actor {0} has an invalid HLOD Layer {1}"),
											FText::FromString(GetFullActorName(ActorDescView)), 
											FText::FromString(ActorDescView.GetHLODLayer().ToString()));

		AssetWarning(CurrentAsset, CurrentWarning);
	}
}

void UWorldPartitionChangelistValidator::OnUnsupportedHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView, const UHLODLayer* HLODLayer, EHLODLayerUnsupportedReason Reason)
{
	if (Filter(ActorDescView))
	{
		FText CurrentWarning;
		FText LayerTypeText = HLODLayer ? StaticEnum<EHLODLayerType>()->GetDisplayNameTextByValue((int64)HLODLayer->GetLayerType()) : FText::FromString(TEXT("Unknown"));

		switch(Reason)
		{
		case EHLODLayerUnsupportedReason::ActorClassUnsupportedByLayerType:
			CurrentWarning = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.UnsupportedHLODLayer", "Actor {0} of class '{1}' is unsupported by HLOD Layer {2} of type '{3}'"),
				FText::FromString(GetFullActorName(ActorDescView)),
				FText::FromString(ActorDescView.GetActorNativeClass()->GetName()),
				FText::FromString(ActorDescView.GetHLODLayer().ToString()),
				LayerTypeText);
			break;
		};

		AssetWarning(CurrentAsset, CurrentWarning);
	}
}

void UWorldPartitionChangelistValidator::OnRejectedActorDescViewMutator(const IWorldPartitionActorDescInstanceView& ActorDescView, const FActorDescViewMutator& RejectedOverrides, EActorDescViewMutatorRejectedReason Reason)
{
	if (Filter(ActorDescView))
	{
		FText CurrentWarning;
		switch (Reason)
		{
		case EActorDescViewMutatorRejectedReason::DivergesClusterRuntimeGrid:
			CurrentWarning = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.MutatorRejectedRuntimeGrid", "Mutator override rejected: cannot set {0} RuntimeGrid to {1} because it would split a reference-connected cluster"),
				FText::FromString(GetFullActorName(ActorDescView)),
				FText::FromName(RejectedOverrides.RuntimeGrid.GetValue()));
			break;
		case EActorDescViewMutatorRejectedReason::DivergesClusterIsSpatiallyLoaded:
			CurrentWarning = FText::Format(LOCTEXT("DataValidation.Changelist.WorldPartition.MutatorRejectedIsSpatiallyLoaded", "Mutator override rejected: cannot set {0} IsSpatiallyLoaded to {1} because it would split a reference-connected cluster"),
				FText::FromString(GetFullActorName(ActorDescView)),
				FText::FromString(RejectedOverrides.bIsSpatiallyLoaded.GetValue() ? TEXT("true") : TEXT("false")));
			break;
		}

		AssetWarning(CurrentAsset, CurrentWarning);
	}
}

#undef LOCTEXT_NAMESPACE