// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EditorValidatorBase.h"
#include "WorldPartition/ErrorHandling/WorldPartitionStreamingGenerationErrorHandler.h"

#include "WorldPartitionChangelistValidator.generated.h"

#define UE_API DATAVALIDATION_API

class UDataValidationChangelist;
class UActorDescContainerInstance;
class FWorldPartitionActorDesc;

UCLASS(MinimalAPI)
class UWorldPartitionChangelistValidator : public UEditorValidatorBase, public IStreamingGenerationErrorHandler
{
	GENERATED_BODY()

protected:	
	FTopLevelAssetPath RelevantMap;
	TSet<FGuid> RelevantActorGuids;
	TSet<FString> RelevantDataLayerAssets;
	TSet<FString> RelevantExternalPackageDataLayerInstances;
	UObject* CurrentAsset = nullptr;
	bool bSubmittingWorldDataLayers = false;

	UE_API virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	UE_API virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;

	UE_API void ValidateActorsAndDataLayersFromChangeList(UDataValidationChangelist* Changelist);

	// Return true if this ActorDescView is pertinent to the current ChangeList
	UE_API bool Filter(const IWorldPartitionActorDescInstanceView& ActorDescView);

	// Return true if this UDataLayerInstance is pertinent to the current ChangeList
	UE_API bool Filter(const UDataLayerInstance* DataLayerInstance);

	// IStreamingGenerationErrorHandler Interface methods
	UE_API virtual void OnInvalidRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, FName GridName) override;
	UE_API virtual void OnInvalidReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const FGuid& ReferenceGuid, IWorldPartitionActorDescInstanceView* ReferenceActorDescView) override;
	UE_API virtual void OnInvalidReferenceGridPlacement(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) override;
	UE_API virtual void OnInvalidReferenceDataLayers(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView, EDataLayerInvalidReason Reason) override;
	UE_API virtual void OnInvalidWorldReference(const IWorldPartitionActorDescInstanceView& ActorDescView, EWorldReferenceInvalidReason Reason) override;
	UE_API virtual void OnInvalidReferenceRuntimeGrid(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) override;
	UE_API virtual void OnDataLayersLoadFilterMismatch(const IWorldPartitionActorDescInstanceView& ActorDescView) override;
	UE_API virtual void OnInvalidReferenceDataLayerAsset(const UDataLayerInstanceWithAsset* DataLayerInstance) override;
	UE_API virtual void OnInvalidDataLayerAssetType(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerAsset* DataLayerAsset) override;
	UE_API virtual void OnDataLayerHierarchyTypeMismatch(const UDataLayerInstance* DataLayerInstance, const UDataLayerInstance* Parent, EDataLayerHierarchyInvalidReason Reason) override;
	UE_API virtual void OnInvalidWorldDataLayersReference(const AWorldDataLayers* WorldDataLayers, const UDataLayerInstance* DataLayerInstance, const FText& Reason) override;
	UE_API virtual void OnDataLayerAssetConflict(const UDataLayerInstanceWithAsset* DataLayerInstance, const UDataLayerInstanceWithAsset* ConflictingDataLayerInstance) override;
	UE_API virtual void OnActorNeedsResave(const IWorldPartitionActorDescInstanceView& ActorDescView) override;
	UE_API virtual void OnLevelInstanceInvalidWorldAsset(const IWorldPartitionActorDescInstanceView& ActorDescView, FName WorldAsset, ELevelInstanceInvalidReason Reason) override;
	UE_API virtual void OnInvalidActorFilterReference(const IWorldPartitionActorDescInstanceView& ActorDescView, const IWorldPartitionActorDescInstanceView& ReferenceActorDescView) override;
	UE_API virtual void OnInvalidHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView) override;
	UE_API virtual void OnUnsupportedHLODLayer(const IWorldPartitionActorDescInstanceView& ActorDescView, const UHLODLayer* HLODLayer, EHLODLayerUnsupportedReason Reason) override;
	UE_API virtual void OnRejectedActorDescViewMutator(const IWorldPartitionActorDescInstanceView& ActorDescView, const FActorDescViewMutator& RejectedOverrides, EActorDescViewMutatorRejectedReason Reason) override;
};

#undef UE_API
