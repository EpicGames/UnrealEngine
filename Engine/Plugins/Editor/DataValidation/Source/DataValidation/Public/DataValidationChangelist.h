// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "ISourceControlChangelist.h"
#include "ISourceControlChangelistState.h"

#include "DataValidationChangelist.generated.h"

#define UE_API DATAVALIDATION_API

/**
 * Changelist abstraction to allow changelist-level data validation
 */
UCLASS(MinimalAPI, config = Editor)
class UDataValidationChangelist : public UObject
{
	GENERATED_BODY()

public:
	/** Default constructor with nothing */
	UDataValidationChangelist() = default;

	UE_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const override;

	/** Initializes from a list of file states as a pseudo-changelist */
	UE_API void Initialize(TConstArrayView<FSourceControlStateRef> InFileStates);
	/** Initializes from a changelist reference, querying the state from the provider */
	UE_API void Initialize(FSourceControlChangelistPtr InChangelist);
	/** Initializes from an already-queried changelist state */
	UE_API void Initialize(FSourceControlChangelistStateRef InChangelistState);

	static UE_API void GatherDependencies(const FName& InPackageName, TSet<TPair<FName, FName>>& OutDependencies);
	static UE_API FString GetPrettyPackageName(const FName& InPackageName);

	/** Changelist to validate - may be null if this was constructed from a list of files */
	FSourceControlChangelistPtr Changelist;
	
	// Asset files in the changelist 
	TArray<FName> ModifiedPackageNames;
	TArray<FName> DeletedPackageNames;
	
	// Non-asset files in the changelist
	TArray<FString> ModifiedFiles;
	TArray<FString> DeletedFiles;
	
	FText Description;
};

#undef UE_API
