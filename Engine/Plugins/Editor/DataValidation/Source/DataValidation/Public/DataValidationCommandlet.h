// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Commandlets/Commandlet.h"
#include "DataValidationCommandlet.generated.h"

#define UE_API DATAVALIDATION_API

UCLASS(MinimalAPI, CustomConstructor)
class UDataValidationCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	UDataValidationCommandlet(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		LogToConsole = false;
	}

	// Begin UCommandlet Interface
	UE_API virtual int32 Main(const FString& FullCommandLine) override;
	// End UCommandlet Interface

	// Run the commandlet validation logic without actually launching the editor in a commandlet mode.
	static UE_API bool ValidateData(const FString& FullCommandLine);

protected:
	UE_API bool ValidateDataImpl(const FString& FullCommandLine);

	// Parses the commandline and processes any found tokens/switch/parameters.
	UE_API void ProcessCommandLine(const FString& FullCommandLine);
	UE_API virtual void ProcessCommandLine(const TArray<FString>& Tokens, const TArray<FString>& Switches, const TMap<FString, FString>& Params);

	UE_API virtual bool GetAssetsToValidate(class IAssetRegistry& AssetRegistry, TArray<FAssetData>& OutAssetDataList);
	UE_API virtual void FilterAssetsToValidate(TArray<FAssetData>& AssetDataList);

	UE_API virtual bool ShouldLoadDefaultEditorModules(const TArray<FAssetData>& AssetDataList);

	UE_API virtual void SetupValidationSettings(struct FValidateAssetsSettings& Settings);
	UE_API virtual void ValidateAssets(const TArray<FAssetData>& AssetDataList, const struct FValidateAssetsSettings& InSettings, struct FValidateAssetsResults& OutResults);
	UE_API virtual bool ProcessValidationResults(struct FValidateAssetsResults& Results);

	/** Checks if the AssetRegistry is performing its initial startup scan, and if so waits for the scan to complete. */
	static UE_API void WaitForAssetRegistry();

protected:
	FString AssetTypeFilterString;
	bool bIncludeOnlyOnDiskAssets = false;
	bool bWithoutEngine = true;
};

#undef UE_API
