// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/TokenizedMessage.h"
#include "Misc/DataValidation/Fixer.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API DATAVALIDATION_API

namespace UE::DataValidation
{

/** Functor which always returns EFixApplicability::CanBeApplied. */
struct FFixAlwaysApplicable
{
	EFixApplicability operator()() const
	{
		return EFixApplicability::CanBeApplied;
	}
};

/**
 * IFixer created out of a pair of lambdas.
 * This can be constructed conveniently using MakeFix.
 */
template <typename GetApplicabilityT, typename ApplyFixT>
struct TLambdaFixer : IFixer
{
	TLambdaFixer(GetApplicabilityT&& GetApplicabilityImpl, ApplyFixT&& ApplyFixImpl)
		: GetApplicabilityImpl(MoveTemp(GetApplicabilityImpl))
		, ApplyFixImpl(MoveTemp(ApplyFixImpl))
	{}

	virtual EFixApplicability GetApplicability(int32 FixIndex) const override
	{
		return GetApplicabilityImpl();
	}

	virtual FFixResult ApplyFix(int32 FixIndex) override
	{
		return ApplyFixImpl();
	}

private:
	GetApplicabilityT GetApplicabilityImpl;
	ApplyFixT ApplyFixImpl;
};

/** Makes an IFixer from applicability and application lambdas. */
template <typename GetApplicabilityT, typename ApplyFixT>
TSharedRef<TLambdaFixer<GetApplicabilityT, ApplyFixT>> MakeFix(GetApplicabilityT&& GetApplicability, ApplyFixT&& ApplyFix)
{
	return MakeShared<TLambdaFixer<GetApplicabilityT, ApplyFixT>>(MoveTemp(GetApplicability), MoveTemp(ApplyFix));
}

/** Makes an always applicable IFixer from an application lambda. */
template <typename ApplyFixT>
TSharedRef<TLambdaFixer<FFixAlwaysApplicable, ApplyFixT>> MakeFix(ApplyFixT&& ApplyFix)
{
	return MakeShared<TLambdaFixer<FFixAlwaysApplicable, ApplyFixT>>(FFixAlwaysApplicable{}, MoveTemp(ApplyFix));
}

/**
 * IFixer which wraps another fixer and makes it single use only.
 * All fixers which are not idempotent will want to use this.
 */
struct FSingleUseFixer : IFixer
{
	TSharedPtr<IFixer> Inner;
	TSet<int32> UsedFixes;

	UE_API virtual EFixApplicability GetApplicability(int32 FixIndex) const override;
	UE_API virtual FFixResult ApplyFix(int32 FixIndex) override;

	static UE_API TSharedRef<FSingleUseFixer> Create(TSharedRef<IFixer> Inner);
};

/** IFixer which wraps another fixer in a set of dependencies. */
struct FObjectSetDependentFixer : IFixer
{
	TSharedPtr<IFixer> Inner;
	TArray<TWeakObjectPtr<>> Dependencies;

	UE_API virtual EFixApplicability GetApplicability(int32 FixIndex) const override;
	UE_API virtual FFixResult ApplyFix(int32 FixIndex) override;

	static UE_API TSharedRef<FObjectSetDependentFixer> Create(TSharedRef<IFixer> Inner, TArray<TWeakObjectPtr<>> Dependencies);
};

/** IFixer which wraps another fixer and makes it auto-save the asset after the fix is applied. */
struct FAutoSavingFixer : IFixer
{
	TSharedPtr<IFixer> Inner;

	UE_API virtual EFixApplicability GetApplicability(int32 FixIndex) const override;
	UE_API virtual FFixResult ApplyFix(int32 FixIndex) override;

	static UE_API TSharedRef<FAutoSavingFixer> Create(TSharedRef<IFixer> Inner);
};

/** IFixer which wraps another fixer and makes it validate any assets touched by the fix. */
struct FValidatingFixer : IFixer
{
	TSharedPtr<IFixer> Inner;

	UE_API virtual EFixApplicability GetApplicability(int32 FixIndex) const override;
	UE_API virtual FFixResult ApplyFix(int32 FixIndex) override;

	static UE_API TSharedRef<FValidatingFixer> Create(TSharedRef<IFixer> Inner);
};

/**
 * IFixer made out of many smaller fixers. Only one of the fixes in the set may be applied.
 * Once applied, the remaining fixers become non-applicable.
 */
struct FMutuallyExclusiveFixSet
{
	UE_API FMutuallyExclusiveFixSet();

	/** Add a fixer to the set. */
	UE_API void Add(const FText& Label, TSharedRef<IFixer> Inner);
	/**
	 * Transform all fixes.
	 * The callback will be called for each fix and should return a new, transformed fix.
	 */
	UE_API void Transform(const TFunctionRef<TSharedRef<IFixer>(TSharedRef<IFixer>)>& Callback);
	/** Generate FFixTokens from the set. These tokens can then be added to a message. */
	UE_API void CreateTokens(const TFunctionRef<void(TSharedRef<FFixToken>)>& Callback) const;

private:
	struct FSharedData
	{
		int32 AppliedFix = INDEX_NONE;
	};
	TSharedPtr<FSharedData> SharedData;

	/** Fix belonging to the mutually exclusive fix set. */
	struct FFixer : IFixer
	{
		TSharedPtr<IFixer> Inner;
		TSharedPtr<FSharedData> SharedData;

		virtual EFixApplicability GetApplicability(int32 FixIndex) const override;
		virtual FFixResult ApplyFix(int32 FixIndex) override;
	};

	/**
	 * Token that is about to be created from the set.
	 * Note that tokens are not created immediately to allow Transform to work.
	 */
	struct FQueuedToken
	{
		FText Label;
		TSharedRef<FFixer> Fixer;
	};

	TArray<FQueuedToken> QueuedTokens;
};

}

#undef UE_API
