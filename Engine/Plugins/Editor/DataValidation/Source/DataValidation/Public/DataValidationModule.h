// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "Misc/DataValidation.h"

struct FAssetData;

/**
 * The public interface to this module
 */
class IDataValidationModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IDataValidationModule& Get()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_IDataValidationModule_Get);
		static IDataValidationModule& Singleton = FModuleManager::LoadModuleChecked< IDataValidationModule >("DataValidation");
		return Singleton;
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_IDataValidationModule_IsAvailable);
		return FModuleManager::Get().IsModuleLoaded( "DataValidation" );
	}

	/** Validates selected assets and opens a window to report the results. If bValidateDependencies is true it will also validate any assets that the selected assets depend on. */
	virtual void ValidateAssets(const TArray<FAssetData>& SelectedAssets, bool bValidateDependencies, const EDataValidationUsecase InValidationUsecase) = 0;
};
