// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "EditorValidator_Material.h"

#include "DataValidationSettings.generated.h"

#define UE_API DATAVALIDATION_API

/**
 * Project-wide settings for data validation
 */
UCLASS(MinimalAPI, config = Editor)
class UDataValidationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Default constructor that sets up CDO properties */
	UE_API UDataValidationSettings();

	/** Whether or not to validate assets on save */
	UPROPERTY(EditAnywhere, Config, Category="Data Validation")
	uint32 bValidateOnSave : 1 = true;
	
	/** Whether or not to load & validate assets in changelists by default */
	UPROPERTY(EditAnywhere, Config, Category="Data Validation")
	uint32 bLoadAssetsWhenValidatingChangelists : 1 = true;

	UPROPERTY(EditAnywhere, Config, Category="Data Validation")
	bool bEnableMaterialValidation = true;

	UPROPERTY(Config)
	bool bMaterialValidationAllowCompilingShaders = true;
	
	UPROPERTY(Config)
	bool bMaterialValidationShowWarningsWhenNotCompilingShaders = true;

	UPROPERTY(EditAnywhere, Config, Category="Data Validation", meta=(EditCondition="bEnableMaterialValidation"))
	TArray<FMaterialEditorValidationPlatform> MaterialValidationPlatforms;
};

#undef UE_API
