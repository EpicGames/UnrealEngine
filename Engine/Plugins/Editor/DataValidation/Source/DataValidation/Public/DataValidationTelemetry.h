// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This header is intended to be included by modules which do not link against DataValidation for access to telemetry data.

#include "Misc/DataValidation.h"
#include "Misc/Guid.h"
#include "UObject/UObjectGlobals.h"

struct FValidateAssetsResults;
struct FValidateAssetsSettings;

namespace UE::Telemetry::DataValidation
{
	enum class EValidationMode
	{
		Unknown,
		Editor,
		Commandlet,
	};

	struct FValidatorStatistics
	{
		int32 AssetsValidated = 0;
		int32 AssetsAddedForValidation = 0;
	};

	// Telemetry for an operation where we validate some assets (or some changelists)
	struct FValidateAssets
	{
		static inline constexpr FGuid TelemetryID{0x0e4e8035, 0xdce94c98, 0x94f64b35, 0xef12b049};

		FValidateAssets() = default;
		FValidateAssets(const FValidateAssetsSettings& InSettings);

		void SetResults(EDataValidationResult InOverallResult, const FValidateAssetsResults& InResults);

		double Duration = 0.0f;
		EValidationMode Mode = EValidationMode::Unknown;	
		EDataValidationResult Result = EDataValidationResult::NotValidated;
		
		// From FValidateAssetsSettings
		bool bSkipExcludedDirectories = false;
		EDataValidationUsecase ValidationUsecase = EDataValidationUsecase::None;
		bool bLoadAssetsForValidation = false;
		bool bUnloadAssetsLoadedForValidation = false;
		bool bLoadExternalObjectsForValidation = false;
		bool bCaptureAssetLoadLogs = false;
		bool bCaptureLogsDuringValidation = false;
		bool bCaptureWarningsDuringValidationAsErrors = false;
		int32 MaxAssetsToValidate = 0;
		bool bValidateReferencersOfDeletedAssets = false;

		int NumRequested = 0;
		int NumExternalObjects = 0;
		int NumChecked = 0;
		int NumValid = 0;
		int NumInvalid = 0;
		int NumSkipped = 0;
		int NumWarnings = 0;
		int NumUnableToValidate = 0;
		bool bAssetLimitReached = false;
		TMap<FTopLevelAssetPath, FValidatorStatistics> ValidatorStatistics;
	};
};