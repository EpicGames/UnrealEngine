// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"
#include "DirtyFilesChangelistValidator.generated.h"

#define UE_API DATAVALIDATION_API

class UPackage;

/**
* Validates there is no unsaved files in the changelist about to be submitted.
*/
UCLASS(MinimalAPI)
class UDirtyFilesChangelistValidator : public UEditorValidatorBase
{
	GENERATED_BODY()

protected:
	UE_API virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	UE_API virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
};

#undef UE_API
