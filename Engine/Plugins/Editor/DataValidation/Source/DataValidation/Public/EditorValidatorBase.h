// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "UObject/Package.h"

#include "EditorValidatorBase.generated.h"

#define UE_API DATAVALIDATION_API

enum class EDataValidationUsecase : uint8;
namespace EMessageSeverity { enum Type : int; };
struct FAssetData;
class FDataValidationContext;
class FTokenizedMessage;
class UDataValidationChangelist;
/*
* The EditorValidatorBase is a class which verifies that an asset meets a specific ruleset.
* It should be used when checking engine-level classes, as UObject::IsDataValid requires
* overriding the base class. You can create project-specific version of the validator base,
* with custom logging and enabled logic.
*
* C++ and Blueprint validators will be gathered on editor start, while python validators need
* to register themselves
*/
UCLASS(MinimalAPI, Abstract, Blueprintable, meta = (ShowWorldContextPin), Config=Editor)
class UEditorValidatorBase : public UObject
{
	GENERATED_BODY()

public:
	// Validation entry point - will dispatch to native and/or BP validation as appropriate and return results.
	// If validator doesn't want to validate this asset/in this context, it will return NotValidated which is acceptable.
	UE_API EDataValidationResult ValidateLoadedAsset(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& Context);

	/** Override this to return additional assets that need validation because of other changes in a changelist. e.g. referencers of a specific type may be broken by edits to a dependency. */
	virtual TArray<FAssetData> GetAssetsToValidateFromChangelist(UDataValidationChangelist* InChangelist, FDataValidationContext& InContext) const
	{
		return {};
	}

	/** Override this to do any state cleanup or add additional validation messages after all per asset validation is complete. */
	virtual void PostAssetValidation(TArray<TSharedRef<FTokenizedMessage>>& OutMessages) {}

protected:
	/** 
	 * Override this to determine whether or not you can use this validator given this context.
	 * Context can be used to add errors if validation cannot be performed because of some issue 
	 */
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InObject, FDataValidationContext& InContext) const
	{
		return false;
	}
	
	/** Override this to validate in C++ with access to FDataValidationContext */
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& Context)
	{
		return EDataValidationResult::NotValidated;
	}

	/** Override this to determine whether or not you can use this validator given this usecase */
	UFUNCTION(BlueprintNativeEvent, Category = "Asset Validation", DisplayName="Can Validate")
	UE_API bool K2_CanValidate(const EDataValidationUsecase InUsecase) const;
	
	/** Override this to determine whether or not you can validate a given asset with this validator */
	UFUNCTION(BlueprintNativeEvent, Category = "Asset Validation", DisplayName="Can Validate Asset")
	UE_API bool K2_CanValidateAsset(UObject* InAsset) const;
	UE_API virtual bool K2_CanValidateAsset_Implementation(UObject* InAsset) const;

	/** Override this in blueprint to validate assets */
	UFUNCTION(BlueprintNativeEvent, Category = "Asset Validation", DisplayName="Validate Loaded Asset")
	UE_API EDataValidationResult K2_ValidateLoadedAsset(UObject* InAsset);
	UE_API virtual EDataValidationResult K2_ValidateLoadedAsset_Implementation(UObject* InAsset);
	
public:
	/** Marks the validation as failed and adds an error message. */
	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API void AssetFails(const UObject* InAsset, const FText& InMessage);

	/** Marks the validation as successful. Failure to call this will report the validator as not having checked the asset. */
	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API void AssetPasses(const UObject* InAsset);

	/** Adds a message to this validation but doesn't mark it as failed. */
	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API void AssetWarning(const UObject* InAsset, const FText& InMessage);
	
	/** 
	 * Add a tokenized message to the validation results. If the severity is error, marks the validation as failed.
	 */
	UE_API TSharedRef<FTokenizedMessage> AssetMessage(const FAssetData& InAssetData, EMessageSeverity::Type InSeverity, const FText& InText = {});
	UE_API TSharedRef<FTokenizedMessage> AssetMessage(EMessageSeverity::Type InSeverity, const FText& InText = {});

	virtual bool IsEnabled() const
	{
		return bIsEnabled && !bIsConfigDisabled;
	}

	UE_API void ResetValidationState();

	bool IsValidationStateSet() const 
	{
		return ValidationResult != EDataValidationResult::NotValidated;
	}

	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	EDataValidationResult GetValidationResult() const 
	{
		return ValidationResult;
	}

	UE_API const TArray<FText>& GetAllWarnings() const;
	
	UE_API void AddLegacyValidationErrors(TArray<FText> InErrors);
	UE_API EDataValidationResult ExtractValidationState(FDataValidationContext& InOutContext) const;

protected:
	// Legacy entry points from old interface 
	UE_DEPRECATED("5.4", "CanValidate_Implementation(const EDataValidationUsecase InUsecase) is deprecated, override CanValidateAsset_Implementation(UObject* InObject, FDataValidationContext& InContext) instead")
	virtual bool CanValidate_Implementation(const EDataValidationUsecase InUsecase) const 
	{
		return false; 
	}
	UE_DEPRECATED("5.4", "CanValidateAsset_Implementation(UObject* InAsset) is deprecated, override CanValidateAsset_Implementation(UObject* InObject, FDataValidationContext& InContext) instead")
	virtual bool CanValidateAsset_Implementation(UObject* InAsset) const 
	{
		 return true; 
	}
	UE_DEPRECATED("5.4", "ValidateLoadedAsset_Implementation(UObject* InAsset, TArray<FText>& ValidationErrors) is deprecated, override ValidateLoadedAsset_Implementation(UObject* InAsset, FDataValidationContext& Context) instead")
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(UObject* InAsset, TArray<FText>& ValidationErrors) 
	{
		return EDataValidationResult::NotValidated;
	}

protected:
	UPROPERTY(EditAnywhere, Category = "Asset Validation", meta = (BlueprintProtected = "true"))
	bool bIsEnabled = true;

	/* Allows disabling validators from config without checking in assets/modifying code. */
	UPROPERTY(Config)
	bool bIsConfigDisabled = false;

	/* Whether we should also print out the source validator when printing validation errors. */
	UPROPERTY(EditAnywhere, Category = "Asset Validation", meta = (BlueprintProtected = "true"))
	bool bOnlyPrintCustomMessage = false;

private:
	EDataValidationResult ValidationResult = EDataValidationResult::NotValidated;
	TArray<FText> AllWarnings;
	TArray<FText> AllErrors;
	TArray<TSharedRef<FTokenizedMessage>> AllMessages;

	UPROPERTY(Transient)
	TObjectPtr<UObject> CurrentObjectBeingValidated = nullptr;
	const FAssetData* CurrentAssetBeingValidated = nullptr;
};

#undef UE_API
