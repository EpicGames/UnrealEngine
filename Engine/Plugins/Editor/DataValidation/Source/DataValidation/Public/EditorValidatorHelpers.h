// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "ProfilingDebugging/CookStats.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"

#define UE_API DATAVALIDATION_API

class FMessageLog;
class FText;
class FDataValidationContext;
class FWildcardString;
class UObject;

struct FAnalyticsEventAttribute;
struct FAssetData;
struct FTopLevelAssetPath;
struct FValidatorStatistics;

namespace EMessageSeverity
{
    enum Type : int;
}

namespace UE::DataValidation
{    
    // Utility to capture log messages from all threads during data validation so they can be filtered
    // and passed as validation errors if desired.
    struct FLogMessageGathererImpl;
    struct FScopedLogMessageGatherer
    {
        UE_API FScopedLogMessageGatherer(bool bEnabled = true);
        UE_API ~FScopedLogMessageGatherer();

        FScopedLogMessageGatherer(const FScopedLogMessageGatherer&) = delete;
        FScopedLogMessageGatherer& operator=(const FScopedLogMessageGatherer&) = delete;

        FScopedLogMessageGatherer(FScopedLogMessageGatherer&&) = delete;
        FScopedLogMessageGatherer& operator=(FScopedLogMessageGatherer&&) = delete;

        /** Returns the current gather constructed on this thread in an enclosing scope if any */
        static UE_API FScopedLogMessageGatherer* GetCurrentThreadGatherer();
        
        /** 
         * Stop gathering logs and return the warnings and errors gathered so far
         */
        UE_API void Stop(TArray<FString>& OutWarnings, TArray<FString>& OutErrors);

        /** 
         * Tell this scope to ignore certain log messages 
         * @return A tuple that can be passed to RemoveIgnoreCategories to reverse this 
         */
        UE_API TTuple<int32, int32> AddIgnoreCategories(TConstArrayView<FName> NewCategories);

        /** 
         * Tell this scope to stop ignoring certain log messages.
         * @param Range The return value from a previous call to AddIgnoreCategories 
         */
        UE_API void RemoveIgnoreCategories(TTuple<int32, int32> Range);

        /** 
         * Tell this scope to ignore certain log messages 
         * @return A tuple that can be passed to RemoveIgnorePatterns to reverse this 
         */
        UE_API TTuple<int32, int32> AddIgnorePatterns(TConstArrayView<FWildcardString> NewPatterns);

        /** 
         * Tell this scope to stop ignoring certain log messages.
         * @param Range The return value from a previous call to AddIgnorePatterns 
         */
        UE_API void RemoveIgnorePatterns(TTuple<int32, int32> Range);
        
        /** 
         * Delegate that may be bound by a project to set a default list of log categories to ignore 
         * during asset validation
         */
        static UE_API TDelegate<TArray<FName>(void)>& GetDefaultIgnoreCategoriesDelegate();

        /**
         * Delegate that may be bound by a project to set a default list of log patterns to ignore 
         * during asset validation
         */
        static UE_API TDelegate<TArray<FWildcardString>(void)>& GetDefaultIgnorePatternsDelegate();
        
    protected:
        FScopedLogMessageGatherer* Previous = nullptr;
        FLogMessageGathererImpl* Impl = nullptr;
    };
    
    // Utility to tell the closest enclosing FScopedLogMessageGatherer to ignore certain messages
    struct FScopedIgnoreLogMessages
    {
        UE_API FScopedIgnoreLogMessages(TConstArrayView<FWildcardString> Patterns, TConstArrayView<FName> Categories);
        UE_API ~FScopedIgnoreLogMessages();
        
    protected:
        TTuple<int32, int32> PatternsRange = {0, 0};
        TTuple<int32, int32> CategoriesRange = {0, 0};
    };

    // Helper to add asset validation messages from a validation context to a message log
    DATAVALIDATION_API void AddAssetValidationMessages(
        const FAssetData& ForAsset,
        FMessageLog& Log,
        const FDataValidationContext& InContext);

    // Helper to add asset validation messages from a validation context to a message log
    DATAVALIDATION_API void AddAssetValidationMessages(
        FMessageLog& Log,
        const FDataValidationContext& InContext);

    // Helper to format asset validation messages where the messages may contain references to assets that should be replaced by clickable tokens.
    DATAVALIDATION_API void AddAssetValidationMessages(
        const FAssetData& ForAsset,
        FMessageLog& Log,
        EMessageSeverity::Type Severity,
        TConstArrayView<FText> Messages);

	/** Prepend the given UTC timestamp (or UtcNow if none provided) to the given base title to provide a pseudo-unique title, eg) "[Timestamp] BaseTitle" */
	UE_API FText MakeTimestampedMessageLogPageTitle(const FText& BaseTitle);
	UE_API FText MakeTimestampedMessageLogPageTitle(const FText& BaseTitle, const FDateTime& UtcTimestamp);

	UE_API void ValidationStatsToAnalyticEventData(const TMap<FTopLevelAssetPath, FValidatorStatistics>& ValidationStats, TArray<FAnalyticsEventAttribute>& OutEventAttributes);
#if ENABLE_COOK_STATS
	UE_API void ValidationStatsToAnalyticEventData(const TMap<FTopLevelAssetPath, FValidatorStatistics>& ValidationStats, FCookStatsManager::AddStatFuncRef AddStat);
#endif
}

#undef UE_API
