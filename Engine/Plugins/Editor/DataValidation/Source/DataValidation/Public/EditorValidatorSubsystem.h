// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"

#include "DataValidationModule.h"
#include "DataValidationSettings.h"

#include "AssetRegistry/AssetData.h"
#include "UObject/Package.h"
#include "EditorValidatorSubsystem.generated.h"

#define UE_API DATAVALIDATION_API

class UDataValidationChangelist;
class UEditorValidatorBase;
class UEditorValidatorSubsystem;
struct FDirectoryPath;
class FMessageLog;
struct FARFilter;
struct FAssetData;
class ISourceControlChangelist;
struct FSourceControlValidationDelegateResult;
typedef TSharedPtr<class ISourceControlChangelist, ESPMode::ThreadSafe> FSourceControlChangelistPtr;
typedef TSharedRef<class ISourceControlState, ESPMode::ThreadSafe> FSourceControlStateRef;

enum class EModuleChangeReason;

namespace EMessageSeverity
{
	enum Type : int;
}

DECLARE_LOG_CATEGORY_EXTERN(LogContentValidation, Log, All);

struct FValidateAssetsExternalObject
{
	/** Package Name */
	FName PackageName;

	/** Asset Name */
	FName AssetName;
};

USTRUCT(BlueprintType)
struct FValidateAssetsDetails
{
	GENERATED_BODY()

	/** Package Name */
	FName PackageName;

	/** Asset Name */
	FName AssetName;

	/** Validation Result */
	EDataValidationResult Result = EDataValidationResult::Valid;

	/** Validation Errors */
	TArray<FText> ValidationErrors;

	/** Validation Warnings */
	TArray<FText> ValidationWarnings;
	
	/** Rich validation messages including tokens that can generate images or hyperlinks */
	TArray<TSharedRef<FTokenizedMessage>> ValidationMessages;

	/** List of external objects for this asset*/
	TArray<FValidateAssetsExternalObject> ExternalObjects;
};

USTRUCT(BlueprintType)
struct FValidatorStatistics
{
	GENERATED_BODY()

	/** Number of assets validated by this validator */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Validation")
	int32 AssetsValidated = 0;

	/** Number of assets that this validator marked as needing validation e.g. because a dependency changed */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Validation")
	int32 AssetsAddedForValidation = 0;

	/** Total time (in seconds) that this validator spent validating assets */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	double TotalDurationSeconds = 0.0;
};

USTRUCT(BlueprintType)
struct FValidateAssetsResults
{
	GENERATED_BODY()

	FValidateAssetsResults() = default;
	
	/** Total amount of assets that were gathered to validate, not including external objects such as actors. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumRequested = 0;

	/** Total amount of external objects to validate with their owning asset (e.g. actors with maps). */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumExternalObjects = 0;

	/** Amount of tested assets */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumChecked = 0;

	/** Amount of assets without errors or warnings */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumValid = 0;

	/** Amount of assets with errors */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumInvalid = 0;

	/** Amount of assets skipped */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumSkipped = 0;

	/** Amount of assets with warnings */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumWarnings = 0;

	/** Amount of assets that could not be validated */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	int NumUnableToValidate = 0;
	
	/** True if FValidateAssetSettings::MaxAssetsToValidation was reached */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	bool bAssetLimitReached = false;

	/** Per-validator statistics on assets validated and assets added for validation */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	TMap<FTopLevelAssetPath, FValidatorStatistics> ValidatorStatistics;

	/** 
	 * Per asset details
	 * Indexed by object path
	 * Only returned if FValidateAssetsSettings::bCollectPerAssetDetails is true.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Validation")
	TMap<FString, FValidateAssetsDetails> AssetsDetails;

	/** Additional validator tokenized messages not specific to a single asset. */
	TArray<TSharedRef<FTokenizedMessage>> ValidatorMessages;
};

USTRUCT(BlueprintType)
struct FValidateAssetsSettings
{
	GENERATED_BODY()

	UE_API FValidateAssetsSettings();
	UE_API ~FValidateAssetsSettings();

	/** If true, will not validate files in excluded directories */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bSkipExcludedDirectories = true;

	/** If true, will add notifications for files with no validation and display even if everything passes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bShowIfNoFailures = true;

	/** If true, will add an FValidateAssetsDetails for each asset to the results */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bCollectPerAssetDetails = false;

	/** The usecase requiring datavalidation */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	EDataValidationUsecase ValidationUsecase = EDataValidationUsecase::None;

	/** If false, unloaded assets will get skipped from validation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bLoadAssetsForValidation = true;

	/** If true, will attempt to unload assets which were previously unloaded, and loaded for validation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bUnloadAssetsLoadedForValidation = false;
	
	/** 
	 * If false, external objects (e.g. actors stored separately from maps) will not be loaded when 
	 * validating their associated asset (e.g. the map) 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bLoadExternalObjectsForValidation = true;

	/** If true, captures log warnings and errors from loading assets for validation and reports them as validation results */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bCaptureAssetLoadLogs = true;

	/** If true, captures log warnings and errors from other operations that happen during validation and adds them to validation results */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bCaptureLogsDuringValidation = true;
	
	/** If true, captured log warnings during validation are added to the validation results as errors (requires bCaptureLogsDuringValidation) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	bool bCaptureWarningsDuringValidationAsErrors = false;

	/** Maximum number of assets to attempt to validate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Validation")
	int32 MaxAssetsToValidate = MAX_int32;
	
	/** Should we validate referencers of deleted assets in changelists */
	UPROPERTY(EditAnywhere, BlueprintReadwrite, Category = "Validation")
	bool bValidateReferencersOfDeletedAssets = true;

	/**
	 * Minimum severity of validation messages to make the message log visible after validation. 
	 * Defaults to warning, can be disabled by emptying the optional.
	 */
	TOptional<EMessageSeverity::Type> ShowMessageLogSeverity;

	/** The name of the message log to use for warnings/errors/etc */
	FName MessageLogName;

	/** Title of message log page to use for warnings/errors/etc */
	FText MessageLogPageTitle;

	/** Show progress window */
	bool bSilent = false;
};

/**
 * Utility to disable "validate on save" in UEditorValidatorSubsystem for the lifetime of this object.
 * Calls UEditorValidatorSubsystem::PushDisableValidateOnSave on construction and UEditorValidatorSubsystem::PopDisableValidateOnSave on destruction.
 */
struct FScopedDisableValidateOnSave
{
public:
	UE_API FScopedDisableValidateOnSave();
	UE_API ~FScopedDisableValidateOnSave();

	UE_NONCOPYABLE(FScopedDisableValidateOnSave);

private:
	UEditorValidatorSubsystem* EditorValidationSubsystem;
};

/**
 * UEditorValidatorSubsystem manages all the asset validation in the engine. 
 * 
 * The first validation handled is UObject::IsDataValid and its overridden functions.
 * Those validations require custom classes and are most suited to project-specific
 * classes. 
 * 
 * The next validation set is of all registered UEditorValidationBases. These validators 
 * have a function to determine if they can validate a given asset, and if they are 
 * currently enabled. They are good candidates for validating engine classes or 
 * very specific project logic.
 * 
 * Finally, this subsystem may be subclassed to change the overally behavior of
 * validation in your project. If a subclass exist in your project module, it will
 * supercede the engine validation subsystem.
 */
UCLASS(MinimalAPI, Config = Editor)
class UEditorValidatorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	UE_API UEditorValidatorSubsystem();

	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection);
	UE_API virtual void Deinitialize();

	/**
	 * Push a new request to temporarily disable "validate on save".
	 * Should be paired with a call to PopDisableValidateOnSave.
	 */
	UE_API void PushDisableValidateOnSave();

	/**
	 * Pop a previous request to temporarily disable "validate on save".
	 * Should be paired with a call to PushDisableValidateOnSave.
	 */
	UE_API void PopDisableValidateOnSave();

	/**
	 * Called to validate assets from either the UI or a commandlet.
	 * Loads the specified assets and runs all registered validators on them.
	 * Populates the message log with errors and warnings with clickable links.
	 * @param InSettings Structure passing context and settings for ValidateAssetsWithSettings
	 * @param OutResults More detailed information about the results of the validate assets command
	 * @returns Number of assets with validation failures or warnings
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API virtual int32 ValidateAssetsWithSettings(
		const TArray<FAssetData>& AssetDataList,
		const FValidateAssetsSettings& InSettings,
		FValidateAssetsResults& OutResults) const;

	/**
	 * Called to validate assets from either the UI or a commandlet.
	 * Loads the specified assets and runs all registered validators on them.
	 * Populates the message log with errors and warnings with clickable links.
	 * @param InSettings Structure passing context and settings for ValidateAssetsWithSettings
	 * @param OutResults More detailed information about the results of the validate assets command
	 * @returns Validation results for the changelist object itself
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API virtual EDataValidationResult ValidateChangelist(
		UDataValidationChangelist* InChangelist,
		const FValidateAssetsSettings& InSettings,
		FValidateAssetsResults& OutResults) const;

	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API virtual EDataValidationResult ValidateChangelists(
		const TArray<UDataValidationChangelist*> InChangelists,
		const FValidateAssetsSettings& InSettings,
		FValidateAssetsResults& OutResults) const;

	/*
	* Adds a validator to the list, making sure it is a unique instance
	*/
	UFUNCTION(BlueprintCallable, Category = "Validation")
	UE_API void AddValidator(UEditorValidatorBase* InValidator);

	/*
	* Removes a validator
	* Should be called during module shutdown if a validator was added.
	*/
	UFUNCTION(BlueprintCallable, Category = "Validation")
	UE_API void RemoveValidator(UEditorValidatorBase* InValidator);

	/**
	 * Iterate the enabled set of validators.
	 * @note Return true to continue iteration, or false to stop.
	 */
	UE_API void ForEachEnabledValidator(TFunctionRef<bool(UEditorValidatorBase* Validator)> Callback) const;

	/**
	 * Runs registered validators on the provided object.
	 * Does not add anything to any FMessageLog tabs.
	 * @return Returns Valid if the object contains valid data; returns Invalid if the object contains invalid data; returns NotValidated if no validations was performed on the object
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API virtual EDataValidationResult IsObjectValid(
		UObject* InObject,
		TArray<FText>& ValidationErrors,
		TArray<FText>& ValidationWarnings,
		const EDataValidationUsecase InValidationUsecase) const;

	/**
	 * Runs registered validators on the provided object.
	 * Does not add anything to any FMessageLog tabs.
	 * @return Returns Valid if the object contains valid data; returns Invalid if the object contains invalid data; returns NotValidated if no validations was performed on the object
	 */
	UE_API virtual EDataValidationResult IsObjectValidWithContext(
		UObject* InObject,
		FDataValidationContext& InContext) const;

	/**
	 * Loads the object referred to by the provided AssetData and runs registered validators on it.
	 * Does not add anything to any FMessageLog tabs.
	 * @return Returns Valid if the object pointed to by AssetData contains valid data; returns Invalid if the object contains invalid data or does not exist; returns NotValidated if no validations was performed on the object
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Validation")
	UE_API virtual EDataValidationResult IsAssetValid(
		const FAssetData& AssetData,
		TArray<FText>& ValidationErrors,
		TArray<FText>& ValidationWarnings,
		const EDataValidationUsecase InValidationUsecase) const;

	/**
	 * Loads the object referred to by the provided AssetData and runs registered validators on it.
	 * Does not add anything to any FMessageLog tabs.
	 * @return Returns Valid if the object pointed to by AssetData contains valid data; returns Invalid if the object contains invalid data or does not exist; returns NotValidated if no validations was performed on the object
	 */
	UE_API virtual EDataValidationResult IsAssetValidWithContext(
		const FAssetData& AssetData,
		FDataValidationContext& InContext) const;

	/**
	 * Called to validate from an interactive save
	 * 
	 * @param AssetDataList List of Assets to validate
	 * @param bProceduralSave True iff this is e.g. a save from the editor, false if it is a procedural change from e.g. cooking
	 */
	UE_API virtual void ValidateOnSave(TArray<FAssetData> AssetDataList, bool bProceduralSave) const;

	/**
	 * Schedule a validation of a saved package, this will activate next frame by default so it can combine them
	 *
	 * @param PackageName Package to validate
	 * @param bProceduralSave True iff this is e.g. a save from the editor, false if it is a procedural change from e.g. cooking
	 */
	UE_API virtual void ValidateSavedPackage(FName PackageName, bool bProceduralSave);
	
	/** 
	 * From a changelist, return a list of assets to validate.
	 * The base implementation returns assets in modified packages in the changelist.
	 * Subclasses may wish to validate additional assets based on files in the changelist 
	 * (e.g. dependencies, relevant code changes, configuration changes, etc)
	 */
	UE_API virtual void GatherAssetsToValidateFromChangelist(
		UDataValidationChangelist* 		InChangelist,
		const FValidateAssetsSettings& 	Settings,
		TSet<FAssetData>& 				OutAssets,
		FDataValidationContext& 		InContext,
		FValidateAssetsResults& 		OutResults) const;

	
	/** 
	 * Returns true if the given asset should be validated at all.
	 * May be overridden to e.g. skip developer/test assets.
	 * Used to determine which assets should be validated when gathering assets from a source such as a changelist.
	 * Will be ignored when an asset is directly passed for validation.
	 */
	UE_API virtual bool ShouldValidateAsset(
		const FAssetData& 				Asset,
		const FValidateAssetsSettings& 	Settings,
		FDataValidationContext& 		InContext) const;

	
	// Utility functions
	
	// Retrieve all assets matching the given filter from the asset registry and then recursively resolve redirectors to produce a single set of assets.
	UE_API TArray<FAssetData> GetAssetsResolvingRedirectors(FARFilter& InFilter);

	/**
	 * Gets the full class object path of the UDataValidationCommandlet class to use when the Editor needs to instantiate
	 * the commandlet to run validation on the entire project. If empty, UDataValidationCommandlet will be used.
	 */
	const FString& GetDataValidationCommandletClassName() const { return DataValidationCommandletClassName; }


protected:
	UE_API bool ShouldValidateOnSave(bool bProceduralSave) const;

	UE_API void CleanupValidators();
	
	UE_API void WaitForAssetCompilationIfNecessary(EDataValidationUsecase InUsecase, bool bShowProgress = true) const;

	/**
	 * @return Returns true if the current Path should be skipped for validation. Returns false otherwise.
	 */
	UE_API virtual bool IsPathExcludedFromValidation(FStringView Path) const;

	/**
	 * Handles validating all pending save packages
	 */
	UE_API void ValidateAllSavedPackages();

	UE_API void RegisterNativeValidators();

	UE_API void RegisterBlueprintValidators();

	/**
	 * Validates changelist before SCC submit operation
	 */
	UE_API void ValidateChangelistPreSubmit(FSourceControlChangelistPtr Changelist, FSourceControlValidationDelegateResult& OutResult) const;

	/**
	 * Validates files before SCC submit operation
	 */
	UE_API void ValidateFilesPreSubmit(TArray<FSourceControlStateRef>& States, FSourceControlValidationDelegateResult& OutResult) const;

	/**
	 * Handle native modules being loaded or unloaded (@see NativeModulesPendingLoad and NativeModulesPendingUnload)
	 */
	UE_API void OnNativeModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason);

	/**
	 * Handle Blueprint assets being added or removed (@see ValidatorClassesPendingLoad)
	 */
	UE_API void OnAssetsAdded(TConstArrayView<FAssetData> InAssets);
	UE_API void OnAssetsRemoved(TConstArrayView<FAssetData> InAssets);
	UE_API void OnAssetsAddedOrRemoved(TConstArrayView<FAssetData> InAssets, TFunctionRef<void(const FTopLevelAssetPath& BPGC)> Callback);

	/**
	 * Apply any pending changes to the list of active validators (@see ValidatorClassesPendingLoad, NativeModulesPendingLoad, and NativeModulesPendingUnload)
	 */
	UE_API void UpdateValidators() const;
	UE_API void UpdateValidators();

	/*
	 * Adds a validator to the list making sure it is a unique instance, but wait for the first use to load the class (for Blueprints).
	 */
	UE_API void AddValidator(const FTopLevelAssetPath& InValidatorClass);

	/*
	 * Remove any validators (active or pending) for the given class.
	 */
	UE_API void RemoveValidator(const FTopLevelAssetPath& InValidatorClass);

private:
	void AddValidator_Internal(UEditorValidatorBase* InValidator);
	void AddValidator_Internal(FTopLevelAssetPath InValidatorClass);
	void RemoveValidator_Internal(UEditorValidatorBase* InValidator);
	void RemoveValidator_Internal(FTopLevelAssetPath InValidatorClass);
protected:
	friend class FDataValidationModule;

	/** 
	 * Validate a set of assets, adding the results to the log/output of a higher level task (e.g. changelist validation).
	 */
	UE_API EDataValidationResult ValidateAssetsInternal(
		FMessageLog& 					DataValidationLog,
		TSet<FAssetData>				Assets,
		const FValidateAssetsSettings& 	InSettings,
		FValidateAssetsResults& 		OutResults
	) const;
	
	UE_API EDataValidationResult ValidateChangelistsInternal(
		TConstArrayView<UDataValidationChangelist*> InChangelists,
		const FValidateAssetsSettings& 				InSettings,
		FValidateAssetsResults& 					OutResults) const;
	
	UE_API void LogAssetValidationSummary(FMessageLog& DataValidationLog, const FValidateAssetsSettings& InSettings, EDataValidationResult Result, const FValidateAssetsResults& Results) const;

	UE_API EDataValidationResult ValidateObjectInternal(
		const FAssetData& InAssetData,
		UObject* InObject,
		FDataValidationContext& InContext,
		TMap<FTopLevelAssetPath, FValidatorStatistics>* ValidatorStatistics = nullptr) const;
protected:
	/**
	 * Directories to ignore for data validation. Useful for test assets
	 */
	UPROPERTY(config)
	TArray<FDirectoryPath> ExcludedDirectories;

	/**
	 * The full class object path of the UDataValidationCommandlet class to use when the Editor needs to instantiate
	 * the commandlet to run validation on the entire project. If empty, UDataValidationCommandlet will be used.
	 */
	UPROPERTY(config)
	FString DataValidationCommandletClassName;

	/**
	 * Whether it should validate assets on save inside the editor
	 */
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use bValidateOnSave on UDataValidationSettings instead."))
	bool bValidateOnSave;

	/** List of saved package names to validate next frame */
	mutable TSet<FName> SavedPackagesToValidate;

	/**
	 * Active validator instances, mapped from their class.
	 * @note Some instances may be null until UpdateValidators has been called (@see ValidatorClassesPendingLoad)
	 */
	UPROPERTY(Transient)
	TMap<FTopLevelAssetPath, TObjectPtr<UEditorValidatorBase>> Validators;

	/**
	 * Set of Blueprint validator classes (from Validators) that have been discovered since the set of validators was last updated, and need to be loaded by the next call to UpdateValidators.
	 */
	TSet<FTopLevelAssetPath> ValidatorClassesPendingLoad;

	/**
	 * Set of native modules that have been loaded since the set of validators was last updated, and should be queried for new validator classes by the next call to UpdateValidators.
	 */
	TSet<FName> NativeModulesPendingLoad;

	/**
	 * Set of native modules that have been unloaded since the set of validators was last updated, and should be removed from Validators by the next call to UpdateValidators.
	 */
	TSet<FName> NativeModulesPendingUnload;

	bool bHasRegisteredNativeValidators = false;
	bool bHasRegisteredBlueprintValidators = false;

	/** Specifies whether or not to allow Blueprint validators */
	UPROPERTY(config)
	bool bAllowBlueprintValidators;

	/**
	 * Counter used by PushDisableValidateOnSave and PopDisableValidateOnSave to know whether "validate on save" is temporarily disabled.
	 */
	uint32 DisableValidateOnSaveCount = 0;
};

#undef UE_API
