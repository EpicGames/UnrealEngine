// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"

#include "EditorValidator_ActionUtility.generated.h"

class UObject;
class FDataValidationContext; // see UObject.h
struct FAssetData;

// Checks if UActorActionUtility and UAssetActionUtility data is valid
UCLASS()
class UEditorValidator_ActionUtility : public UEditorValidatorBase
{
	GENERATED_BODY()

protected:
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;
};
