// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"
#include "EditorValidator_Localization.generated.h"

#define UE_API DATAVALIDATION_API

/*
* Validates that localized assets (within the L10N folder) conform to a corresponding source asset of the correct type.
* Localized assets that fail this validation will never be loaded as localized variants at runtime.
*/
UCLASS(MinimalAPI)
class UEditorValidator_Localization : public UEditorValidatorBase
{
	GENERATED_BODY()

public:
	UE_API UEditorValidator_Localization();

protected:
	UE_API virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	UE_API virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;

	UE_API const TArray<FString>* FindOrCacheCulturesForLocalizedRoot(const FString& InLocalizedRootPath);

	TMap<FString, TArray<FString>> CachedCulturesForLocalizedRoots;
};

#undef UE_API
