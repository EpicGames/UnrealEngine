// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorValidatorBase.h"
#include "RHIShaderPlatform.h"
#include "RHIFeatureLevel.h"
#include "SceneTypes.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"

#include "EditorValidator_Material.generated.h"

class FText;
class UObject;

// Checks if UMaterial and UMaterialInstance are compiling on all validation shader platforms
UCLASS()
class UEditorValidator_Material : public UEditorValidatorBase
{
	GENERATED_BODY()

public:
	UEditorValidator_Material();

protected:
	virtual bool CanValidateAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) const override;
	virtual EDataValidationResult ValidateLoadedAsset_Implementation(const FAssetData& InAssetData, UObject* InAsset, FDataValidationContext& InContext) override;

private:
	struct FShaderValidationPlatform
	{
		FName ShaderPlatformName;
		EShaderPlatform ShaderPlatform;
		ERHIFeatureLevel::Type FeatureLevel;
		EMaterialQualityLevel::Type MaterialQualityLevel;
	};

	TArray<FShaderValidationPlatform> ValidationPlatforms;

	// Creates a duplicate asset in transient package, will return nullptr if nullptr is passed.
	static UMaterial* DuplicateMaterial(UMaterial* OriginalMaterial);
	static UMaterialInstance* DuplicateMaterialInstance(UMaterialInstance* OriginalMaterialInstance);
};

UCLASS()
class UValidationMaterial : public UMaterial
{
	GENERATED_BODY()

public:
	virtual FMaterialResource* AllocateResource() override;
	virtual bool IsAsset() const override { return false; }
};

USTRUCT()
struct FMaterialEditorValidationShaderPlatform
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category="Data Validation")
	FName Name = MaxRHIShaderPlatformName;

	// if Name == MaxRHIShaderPlatformName, use GMaxRHIShaderPlatform instead
	static constexpr FStringView MaxRHIShaderPlatformNameView = TEXTVIEW("GMaxRHIShaderPlatform");
	static FName MaxRHIShaderPlatformName;

	static FName CustomPropertyTypeLayoutName;
	static void RegisterCustomPropertyTypeLayout();
	static void UnregisterCustomPropertyTypeLayout();
};

UENUM()
enum class EMaterialEditorValidationFeatureLevel : int32
{
	CurrentMaxFeatureLevel = ERHIFeatureLevel::Num + 1	UMETA(DisplayName="Current Max Feature Level"),
	ES3_1 = ERHIFeatureLevel::ES3_1						UMETA(DisplayName="ES3.1"),
	SM5 = ERHIFeatureLevel::SM5,
	SM6 = ERHIFeatureLevel::SM6,
};

UENUM()
enum class EMaterialEditorValidationQualityLevel : uint8
{
	Low = EMaterialQualityLevel::Low,
	Medium = EMaterialQualityLevel::Medium,
	High = EMaterialQualityLevel::High,
	Epic = EMaterialQualityLevel::Epic,
};

USTRUCT()
struct FMaterialEditorValidationPlatform
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, config, Category="Data Validation")
	FMaterialEditorValidationShaderPlatform ShaderPlatform = {};

	UPROPERTY(EditAnywhere, config, Category="Data Validation")
	EMaterialEditorValidationFeatureLevel FeatureLevel = EMaterialEditorValidationFeatureLevel::CurrentMaxFeatureLevel;

	UPROPERTY(EditAnywhere, config, Category="Data Validation")
	EMaterialEditorValidationQualityLevel MaterialQualityLevel = EMaterialEditorValidationQualityLevel::Epic;
};
